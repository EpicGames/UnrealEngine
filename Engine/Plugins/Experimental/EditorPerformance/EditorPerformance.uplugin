{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Editor Performance",
	"Description": "Plugin that provides Editor Performance feedback to developers",
	"Category": "Performance",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": true,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "EditorPerformance",
			"Type": "Editor",
			"LoadingPhase": "PostConfigInit"
		},
		{
			"Name": "StallLogSubsystem",
			"Type": "Editor",
			"LoadingPhase": "PostConfigInit"
		},
		{
			"Name": "CrashDiagnostics",
			"Type": "Editor",
			"LoadingPhase": "PostConfigInit"
		}
	],
	"Plugins": [
		{
			"Name": "EditorDataStorageFeatures",
			"Enabled": true
		}
	]
}