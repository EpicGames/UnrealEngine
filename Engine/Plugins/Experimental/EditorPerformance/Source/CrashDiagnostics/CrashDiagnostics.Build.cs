// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class CrashDiagnostics : ModuleRules
{
	public CrashDiagnostics(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"TypedElementFramework",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"CrashReportCore",
				"InputCore",
				"Slate",
				"SlateCore",
				"TedsQueryStack",
				"TedsTableViewer",
				"ToolMenus",
				"ToolWidgets",
				"UnrealEd",
				"XmlParser",
			}
		);
	}
}