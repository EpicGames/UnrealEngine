// Copyright Epic Games, Inc. All Rights Reserved.

#include "CrashDiagnosticsModule.h"

#include "CrashDescription.h"
#include "CrashDiagnosticsSettings.h"
#include "Editor.h"
#include "PlatformErrorReport.h"
#include "SEditorCrashesPanel.h"
#include "TedsEditorCrashDataStorageFactory.h"
#include "Columns/TedsCrashColumns.h"
#include "Diagnostics/EditorDiagnosticsColumns.h"
#include "DataStorage/Features.h"
#include "Editor/EditorPerformanceSettings.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/FileManager.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Paths.h"
#include "Widgets/Images/SLayeredImage.h"

#define LOCTEXT_NAMESPACE "FCrashDiagnosticsModule"

namespace UE::Editor::CrashDiagnostics::Private
{
	static FAutoConsoleCommand AddCrashesToDataStorageCommand
		(
			TEXT("Editor.CrashDiagnostics.AddCrashesToDataStorage"),
			TEXT("Clear all existing crash rows and reload them from the Saved/Crashes folder into the Data Storage"),
			FConsoleCommandDelegate::CreateLambda([]()
			{
				using namespace UE::Editor::DataStorage;
				if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
				{
					if (FCrashDiagnosticsModule* Module = FCrashDiagnosticsModule::Get())
					{
						DataStorage->RemoveAllRowsWith<FEditorCrashGUIDColumn>();
						GConfig->LoadFile(UCrashDiagnosticsSettings::StaticClass()->GetConfigName());
						Module->LoadCrashDiagnosticsSettings(); // Forces a reset of PreviousSessionStartTime, useful for testing
						Module->AddCrashesToDataStorage(*DataStorage);
					}
				}
			})
		);
}


namespace UE::Editor::CrashDiagnostics
{

void FCrashDiagnosticsModule::StartupModule()
{
	using namespace UE::Editor::DataStorage;

	FEditorDelegates::OnEditorInitialized.AddRaw(this, &FCrashDiagnosticsModule::OnEditorInitialized);
}

void FCrashDiagnosticsModule::ShutdownModule()
{
	FEditorDelegates::OnEditorInitialized.RemoveAll(this);
}

TArray<TSharedRef<FPrimaryCrashProperties>> FCrashDiagnosticsModule::RetrieveSavedCrashes()
{
	TArray<TSharedRef<FPrimaryCrashProperties>> Crashes;

	const FString CrashFolder = FPaths::Combine(*FPaths::ProjectSavedDir(), TEXT("Crashes"));
	if (!FPaths::DirectoryExists(CrashFolder))
	{
		return Crashes;
	}

	IFileManager::Get().IterateDirectory(*CrashFolder,
		[&Crashes](const TCHAR* FilenameOrDirectory, bool bIsDirectory)
		{
			if (bIsDirectory)
			{
				const FPlatformErrorReport ErrorReport(FilenameOrDirectory);
				FString Filename;
				// CrashContext.runtime-xml has the precedence over the WER
				if (ErrorReport.FindFirstReportFileWithExtension(Filename, FGenericCrashContext::CrashContextExtension))
				{
					const bool bIsReadOnly = true;
					Crashes.Add(MakeShared<FCrashContext>(FilenameOrDirectory / Filename, bIsReadOnly));
				}
				else if (ErrorReport.FindFirstReportFileWithExtension(Filename, TEXT(".xml")))
				{
					const bool bIsReadOnly = true;
					Crashes.Add(MakeShared<FCrashWERContext>(FilenameOrDirectory / Filename, bIsReadOnly));
				}
			}
			return true;
		}
	);

#if 0 // FCrashProperties do not load by default. Uncomment this for debugging
	for (TSharedRef<FPrimaryCrashProperties>& Crash : Crashes)
	{
		Crash->EngineModeEx.AsString();
		Crash->PlatformFullName.AsString();
		Crash->CommandLine.AsString();
		Crash->UserName.AsString();
		Crash->LoginId.AsString();
		Crash->EpicAccountId.AsString();
		Crash->GameSessionID.AsString();
		Crash->CallStack.AsString();
		Crash->PCallStack.AsString();
		Crash->PCallStackHashProperty.AsString();
		Crash->SourceContext.AsString();
		Crash->Modules.AsString();
		Crash->UserDescription.AsString();
		Crash->UserActivityHint.AsString();
		Crash->ErrorMessage.AsString();
		Crash->FullCrashDumpLocation.AsString();
		Crash->TimeOfCrash.AsInt64();
		Crash->bAllowToBeContacted.AsBool();
		Crash->CrashReporterMessage.AsString();
		Crash->PlatformCallbackResult.AsInt64();
		Crash->CrashReportClientVersion.AsString();
		Crash->CPUBrand.AsString();
	}
#endif

	return Crashes;
}

void FCrashDiagnosticsModule::AddCrashesToDataStorage(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	const TArray<TSharedRef<FPrimaryCrashProperties>> Crashes = RetrieveSavedCrashes();
	AddCrashesToDataStorage(DataStorage, Crashes);
}

void FCrashDiagnosticsModule::AddCrashesToDataStorage(UE::Editor::DataStorage::ICoreProvider& DataStorage, const TArray<TSharedRef<FPrimaryCrashProperties>>& Crashes)
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::CrashDiagnostics;

	// Table is guaranteed to exist, registered by UTedsEditorCrashDataStorageFactory::RegisterTables
	TableHandle CrashTable = DataStorage.FindTable(DataTableName);
	check(CrashTable != InvalidTableHandle);

	bool bHadCrashLastSession = false;
	for (const TSharedRef<FPrimaryCrashProperties>& Crash : Crashes)
	{
		FMapKey Key = FMapKey(Crash->CrashGUID);
		RowHandle Row = DataStorage.LookupMappedRow(MappingDomain, Key);
		FDateTime CrashTime(Crash->TimeOfCrash.AsInt64());

		if (!DataStorage.IsRowAvailable(Row))
		{
			Row = DataStorage.AddRow(CrashTable);

			FString Folder = FPaths::GetPath(Crash->GetXmlFilepath());
			const FPlatformErrorReport ErrorReport(Folder);
			FEditorCrashFileReportsColumn CrashReports{ .XmlFilePath = Crash->GetXmlFilepath(), .ReportFilePaths = ErrorReport.GetFilesToUpload() };

			if (FString Filename; ErrorReport.FindFirstReportFileWithExtension(Filename, TEXT(".log")))
			{
				CrashReports.LogFilePath = ErrorReport.GetReportDirectory() / MoveTemp(Filename);
			}

			DataStorage.AddColumn<FEditorCrashFileReportsColumn>(Row, MoveTemp(CrashReports));
			DataStorage.AddColumn<FEditorCrashGUIDColumn>(Row, { .CrashGUID = Crash->CrashGUID });
			DataStorage.AddColumn<FEditorCrashTimeColumn>(Row, {
				.TimeOfCrash = CrashTime,
				.TimeOfCrashString = FText::AsDateTime(CrashTime, EDateTimeStyle::Default, EDateTimeStyle::Default, FText::GetInvariantTimeZone())
			});
			DataStorage.AddColumn<FEditorCrashTypeColumn>(Row, { .CrashType = Crash->CrashType });
			if (FString ErrorMessage = Crash->ErrorMessage.AsString().TrimStartAndEnd(); !ErrorMessage.IsEmpty())
			{
				DataStorage.AddColumn<FEditorCrashErrorMessageColumn>(Row, { .ErrorMessage = MoveTemp(ErrorMessage) });
			}
			if (FString CallStack = Crash->CallStack.AsString().TrimStartAndEnd(); !CallStack.IsEmpty())
			{
				DataStorage.AddColumn<FEditorCrashCallStackColumn>(Row, { .CallStack = MoveTemp(CallStack) });
			}
			if (FString SourceContext = Crash->SourceContext.AsString().TrimStartAndEnd(); !SourceContext.IsEmpty())
			{
				DataStorage.AddColumn<FEditorCrashSourceContextColumn>(Row, { .SourceContext = MoveTemp(SourceContext) });
			}
			if (FString UserActivity = Crash->UserActivityHint.AsString().TrimStartAndEnd(); !UserActivity.IsEmpty())
			{
				DataStorage.AddColumn<FEditorCrashUserActivityHintColumn>(Row, { .UserActivityHint = MoveTemp(UserActivity) });
			}
			if (Crash->bIsEnsure)
			{
				DataStorage.AddColumn<FEditorCrashIsEnsureTag>(Row);
			}
			if (Crash->bIsOOM)
			{
				DataStorage.AddColumn<FEditorCrashIsOOMTag>(Row);
			}

			if (PreviousSessionStartTime != FDateTime() && CrashTime > PreviousSessionStartTime)
			{
				bHadCrashLastSession = true;
				if (!DataStorage.HasColumns<FEditorCrashIsNewTag>(Row))
				{
					DataStorage.AddColumn<FEditorCrashIsNewTag>(Row);
				}
			}

			DataStorage.MapRow(MappingDomain, Key, Row);
		}
	}

	{
		// Table is guaranteed to exist, registered by UTedsEditorCrashDataStorageFactory::RegisterTables
		TableHandle GlobalTable = DataStorage.FindTable(GlobalTableName);
		check(GlobalTable != InvalidTableHandle);

		const FMapKey StateKey(LastSessionCrashStateRowKey);
		RowHandle StateRow = DataStorage.LookupMappedRow(MappingDomain, StateKey);
		if (!DataStorage.IsRowAvailable(StateRow))
		{
			StateRow = DataStorage.AddRow(GlobalTable);
			DataStorage.MapRow(MappingDomain, StateKey, StateRow);
		}

		if (bHadCrashLastSession && !DataStorage.HasColumns<FEditorCrashLastSessionTag>(StateRow))
		{
			DataStorage.AddColumn<FEditorCrashLastSessionTag>(StateRow);
			DataStorage.AddColumn<FEditorPerformanceWarningColumn>(StateRow, {
				.Instigator = FName("CrashDiagnostics"),
				.Message = LOCTEXT("LastSessionCrashWarning", "The editor crashed in the previous session.")
			});
		}
	}
}

void FCrashDiagnosticsModule::AddCrashesToDataStorageAsync()
{
	AsyncTask(ENamedThreads::AnyBackgroundThreadNormalTask, []()
	{
		TArray<TSharedRef<FPrimaryCrashProperties>> Crashes = RetrieveSavedCrashes();

		AsyncTask(ENamedThreads::GameThread, [Crashes = MoveTemp(Crashes)]() mutable
		{
			using namespace UE::Editor::DataStorage;
			using namespace UE::Editor::CrashDiagnostics;

			ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
			FCrashDiagnosticsModule* CrashDiagnosticsModule = Get();
			if (!CrashDiagnosticsModule || !DataStorage)
			{
				return;
			}

			TPromise<FDelegateHandle> HandlePromise;
			TSharedFuture<FDelegateHandle> FutureHandle = HandlePromise.GetFuture().Share();
			FDelegateHandle Handle = DataStorage->OnUpdate().AddLambda([Crashes = MoveTemp(Crashes), FutureHandle]()
			{
				ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
				if (DataStorage)
				{
					if (FCrashDiagnosticsModule* CrashDiagnosticsModule = Get())
					{
						CrashDiagnosticsModule->AddCrashesToDataStorage(*DataStorage, Crashes);
					}
					DataStorage->OnUpdate().Remove(FutureHandle.Get());
				}
			});
			HandlePromise.SetValue(Handle);
		});
	});
}

void FCrashDiagnosticsModule::LoadCrashDiagnosticsSettings()
{
	UCrashDiagnosticsSettings* Settings = GetMutableDefault<UCrashDiagnosticsSettings>();
	Settings->LoadConfig();
	PreviousSessionStartTime = Settings->LastSessionStartTime;
}

void FCrashDiagnosticsModule::SaveCrashDiagnosticsSettings()
{
	GetMutableDefault<UCrashDiagnosticsSettings>()->SaveConfig();
}

bool FCrashDiagnosticsModule::HasCrashedLastSession() const
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::Queries;
	ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
	if (!DataStorage)
	{
		return false;
	}

	const UTedsEditorCrashDataStorageFactory* Factory = DataStorage->FindFactory<UTedsEditorCrashDataStorageFactory>();
	const QueryHandle Handle = ensure(Factory) ? Factory->GetHasCrashedLastSessionQueryHandle() : InvalidQueryHandle;
	const FQueryResult Result = Handle != InvalidQueryHandle ? DataStorage->RunQuery(Handle) : FQueryResult();
	return Result.Count > 0;
}

bool FCrashDiagnosticsModule::HasUnreadCrashes() const
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::Queries;
	ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
	if (!DataStorage)
	{
		return false;
	}

	const UTedsEditorCrashDataStorageFactory* Factory = DataStorage->FindFactory<UTedsEditorCrashDataStorageFactory>();
	const QueryHandle Handle = ensure(Factory) ? Factory->GetHasUnreadCrashesQueryHandle() : InvalidQueryHandle;
	const FQueryResult Result = Handle != InvalidQueryHandle ? DataStorage->RunQuery(Handle) : FQueryResult();
	return Result.Count > 0;
}

TSharedRef<SWidget> FCrashDiagnosticsModule::CreateCrashLogPanel()
{
	return SNew(UE::Editor::CrashDiagnostics::SEditorCrashesPanel);
}

template <typename T>
TSharedPtr<T> FindFirstDescendant(const TSharedRef<SWidget>& Root, FName RealWidgetClassName = NAME_None)
{
	if (Root->GetType() == (RealWidgetClassName.IsNone() ? T::StaticWidgetClass().GetWidgetType() : RealWidgetClassName))
	{
		return StaticCastSharedRef<T>(Root);
	}
	if (FChildren* Children = Root->GetChildren())
	{
		for (int32 i = 0; i < Children->Num(); ++i)
		{
			if (TSharedPtr<T> Result = FindFirstDescendant<T>(Children->GetChildAt(i), RealWidgetClassName))
			{
				return Result;
			}
		}
	}
	return nullptr;
}

TSharedRef<SWidget> FCrashDiagnosticsModule::DecorateButtonWithUnreadBadge(TSharedRef<SWidget> InWidget)
{
	FCrashDiagnosticsModule* Module = Get();
	if (!Module)
	{
		return InWidget;
	}

	// SLayeredImage does not have a class
	TSharedPtr<SLayeredImage> LayeredImage = FindFirstDescendant<SLayeredImage>(InWidget, "SLayeredImage");
	if (LayeredImage)
	{
		LayeredImage->AddLayer(TAttribute<const FSlateBrush*>::CreateLambda([]()
		{
			const FCrashDiagnosticsModule* Module = Get();
			return Module ? Module->GetUnreadBadgeIcon() : nullptr;
		}));
	}
	return InWidget;
}

const FSlateBrush* FCrashDiagnosticsModule::GetUnreadBadgeIcon() const
{
	return HasUnreadCrashes() ? FAppStyle::Get().GetBrush("Icons.BadgeModified") : nullptr;
}

void FCrashDiagnosticsModule::OnEditorInitialized(double TimeToInitializeEditor)
{
	LoadCrashDiagnosticsSettings();
	{
		UCrashDiagnosticsSettings* Settings = GetMutableDefault<UCrashDiagnosticsSettings>();
		Settings->LastSessionStartTime = FDateTime::UtcNow();
		Settings->SaveConfig();
	}

	{
		using namespace UE::Editor::DataStorage;
		auto StartupOperations = []()
		{
			if (FCrashDiagnosticsModule* CrashDiagnosticsModule = Get())
			{
				CrashDiagnosticsModule->AddCrashesToDataStorageAsync();
			}
		};

		if (AreEditorDataStorageFeaturesEnabled())
		{
			StartupOperations();
		}
		else
		{
			OnEditorDataStorageFeaturesEnabled().AddLambda([StartupOperations = MoveTemp(StartupOperations)]()
			{
				StartupOperations();
			});
		}
	}
}

}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(UE::Editor::CrashDiagnostics::FCrashDiagnosticsModule, CrashDiagnostics)