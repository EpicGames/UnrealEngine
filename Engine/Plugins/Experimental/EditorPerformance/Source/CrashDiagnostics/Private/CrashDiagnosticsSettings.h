// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "CrashDiagnosticsSettings.generated.h"

UCLASS(MinimalAPI, config=EditorPerProjectUserSettings)
class UCrashDiagnosticsSettings : public UObject
{
	GENERATED_BODY()

public:
	/**
	 * The start time of the last editor session.
	 * Used to detect crashes that occurred during the previous session.
	 */
	UPROPERTY(config)
	FDateTime LastSessionStartTime = FDateTime();
};
