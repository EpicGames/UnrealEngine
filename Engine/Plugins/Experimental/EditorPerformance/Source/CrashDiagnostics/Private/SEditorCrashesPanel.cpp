// Copyright Epic Games, Inc. All Rights Reserved.

#include "SEditorCrashesPanel.h"

#include "CrashDescription.h"
#include "ISettingsModule.h"
#include "TedsEditorCrashDataStorageFactory.h"
#include "TedsQueryNode.h"
#include "TedsQuerySearchNode.h"
#include "ToolMenus.h"
#include "Columns/TedsCrashColumns.h"
#include "DataStorage/Features.h"
#include "Elements/Framework/TypedElementAttributeBinding.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Framework/Docking/TabManager.h"
#include "GenericPlatform/GenericPlatformCrashContext.h"
#include "TedsTableViewer/Public/Widgets/STedsSearchBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SRowDetails.h"
#include "Widgets/STedsTableViewer.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "EditorCrashes"

namespace UE::Editor::CrashDiagnostics
{

const FName SEditorCrashesPanel::SettingsMenuName = "EditorCrashesSettings";

void SEditorCrashesPanel::Construct(const FArguments& InArgs)
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::Queries;

	this->ChildSlot
	[
		SNew(SSplitter)
		.Orientation(Orient_Vertical)
		+ SSplitter::Slot()
		[
			CreateTableWidget()
		]
		+ SSplitter::Slot()
		[
			CreateDetailsWidget()
		]
	];
}

TSharedRef<SWidget> SEditorCrashesPanel::CreateTableWidget()
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::Queries;

	ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
	check(DataStorage);
	TSharedRef<QueryStack::FQueryNode> QueryNode = MakeShared<QueryStack::FQueryNode>(*DataStorage,
		Select()
		.ReadOnly<
			FEditorCrashGUIDColumn,
			FEditorCrashTimeColumn,
			FEditorCrashErrorMessageColumn,
			FEditorCrashTypeColumn
		>()
		.Where()
			.None<FEditorCrashIsEnsureTag>()
		.Compile());

	SearchBox = SNew(STedsSearchBox)
		.InSearchableQueryNode(QueryNode)
		.InSearchableQueryFlags(QueryStack::FQuerySearchNode::ESyncActions::RefreshOnUpdate)
		.OutSearchNode(&SearchNode);

	TSharedPtr<FMetaData> GenericMetaData = MakeShared<FMetaData>();
	GenericMetaData->AddOrSetMutableData("bUseDisplayNameTooltip", true);
	TSharedPtr<FColumnMetaData> ColumnMetaData = MakeShared<FColumnMetaData>(FEditorCrashTimeColumn::StaticStruct(), FColumnMetaData::EFlags::None);
	ColumnMetaData->AddOrSetMutableData("bColumnInitialSortIsAscending", false);

	return SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
		.Padding(0)
		[
			SNew(SScrollBox)
			.Orientation(Orient_Vertical)
			+ SScrollBox::Slot()
			.HAlign(HAlign_Fill)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.Padding(5, 5, 5, 0)
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					[
						SearchBox.ToSharedRef()
					]
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Right)
					.Padding(5, 0, 0, 0)
					.AutoWidth()
					[
						SNew(SComboButton)
						.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButtonWithIcon")
						.HasDownArrow(false)
						.OnGetMenuContent(this, &SEditorCrashesPanel::CreateSettingsMenuWidget)
						.ButtonContent()
						[
							SNew(SImage)
							.Image(FAppStyle::GetBrush("Icons.Settings"))
							.ColorAndOpacity(FSlateColor::UseForeground())
						]
					]
				]
				+ SVerticalBox::Slot()
				.Padding(0, 5, 0, 0)
				.AutoHeight()
				[
					SNew(STedsTableViewer)
					.QueryStack(SearchNode)
					.HeaderWidgetPurpose(UTedsEditorCrashDataStorageFactory::GetHeaderRowPurposeID())
					.CellWidgetPurpose(UTedsEditorCrashDataStorageFactory::GetTableRowPurposeID())
					.OnSelectionChanged(this, &SEditorCrashesPanel::OnSelectionChanged)
					.PrimaryColumn(FEditorCrashTimeColumn::StaticStruct())
					.GenericMetaData(GenericMetaData)
					.ColumnMetaData({MoveTemp(ColumnMetaData)})
					.Columns({
						FEditorCrashTimeColumn::StaticStruct(),
						FEditorCrashTypeColumn::StaticStruct(),
						FEditorCrashErrorMessageColumn::StaticStruct(),
					})
				]
			]
		];
}

TSharedRef<SWidget> SEditorCrashesPanel::CreateDetailsWidget()
{
	using namespace UE::Editor::DataStorage;

	static const TArray<TWeakObjectPtr<const UScriptStruct>> Rows = {
		FEditorCrashGUIDColumn::StaticStruct(),
		FEditorCrashTimeColumn::StaticStruct(),
		FEditorCrashTypeColumn::StaticStruct(),
		FEditorCrashFileReportsColumn::StaticStruct(),
		FEditorCrashErrorMessageColumn::StaticStruct(),
		FEditorCrashUserActivityHintColumn::StaticStruct(),
		FEditorCrashCallStackColumn::StaticStruct(),
		FEditorCrashSourceContextColumn::StaticStruct(),
	};

	TSharedRef<FMetaData> NameMetaData = MakeShared<FMetaData>();
	NameMetaData->AddOrSetMutableData("Font", TEXT("NormalFontBold"));
	TSharedRef<FMetaData> DataMetaData = MakeShared<FMetaData>();
	DataMetaData->AddOrSetMutableData("bUseEditableText", true);

	return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.MinHeight(20.f)
		[
			SNew(SBorder)
			.BorderImage(&FAppStyle::Get().GetWidgetStyle<FTableColumnHeaderStyle>("TableView.Header.Column").NormalBrush)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
				[
					SNew(STextBlock)
					.Text(LOCTEXT("DetailsTitle", "Details"))
				]
			]
		]
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SNew(SBorder)
			.BorderImage(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("TableView.Row").EvenRowBackgroundBrush)
			[
				SNew(SScrollBox)
				.Orientation(Orient_Vertical)
				+ SScrollBox::Slot()
				.Padding(4.0f)
				[
					SAssignNew(CrashRowDetails, SRowDetails)
					.SelectionMode(ESelectionMode::None)
					.Columns(Rows)
					.HeaderRow
					(
						SNew(SHeaderRow)
						.Visibility(EVisibility::Collapsed)
						+ SHeaderRow::Column(SRowDetails::NameColumn)
							.ManualWidth(100.f)
						+ SHeaderRow::Column(SRowDetails::DataColumn)
					)
					.RowPadding(FMargin(0.f, 3.f))
					.NameMetaData(NameMetaData)
					.DataMetaData(DataMetaData)
				]
			]
		];
}

TSharedRef<SWidget> SEditorCrashesPanel::CreateSettingsMenuWidget()
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	if (!ToolMenus)
	{
		return SNullWidget::NullWidget;
	}

	UToolMenu* SettingsMenu = ToolMenus->FindMenu(SettingsMenuName);

	if (!SettingsMenu)
	{
		SettingsMenu = RegisterSettingsMenu();
	}

	if (SettingsMenu)
	{
		return ToolMenus->GenerateWidget(SettingsMenu);
	}
	else
	{
		return SNullWidget::NullWidget;
	}
}

void SEditorCrashesPanel::OnSelectionChanged(UE::Editor::DataStorage::RowHandle RowHandle, ESelectInfo::Type SelectionType)
{
	SelectedRowHandle = RowHandle;
	CrashRowDetails->SetRow(RowHandle);

	if (NewCrashTimer)
	{
		UnRegisterActiveTimer(NewCrashTimer.ToSharedRef());
		NewCrashTimer.Reset();
	}

	using namespace UE::Editor::DataStorage;
	ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);

	if (!ensure(DataStorage) || !DataStorage->IsRowAssigned(RowHandle))
	{
		return;
	}

	if (DataStorage->HasColumns(RowHandle, TArray{FEditorCrashIsNewTag::StaticStruct()}))
	{
		NewCrashTimer = RegisterActiveTimer(MarkAsReadDelaySeconds, FWidgetActiveTimerDelegate::CreateLambda(
			[this, RowHandle](double InCurrentTime, float InDeltaTime)
			{
				ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
				if (SelectedRowHandle == RowHandle && DataStorage)
				{
					DataStorage->RemoveColumn(RowHandle, FEditorCrashIsNewTag::StaticStruct());
				}
				NewCrashTimer.Reset();
				return EActiveTimerReturnType::Stop;
			}));
	}
}

UToolMenu* SEditorCrashesPanel::RegisterSettingsMenu()
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	if (!ToolMenus)
	{
		return nullptr;
	}

	UToolMenu* SettingsMenu = ToolMenus->RegisterMenu(SettingsMenuName);

	FToolMenuSection& SettingsMenuSection = SettingsMenu->AddSection("Settings", LOCTEXT("Settings", "Settings"));

	SettingsMenuSection.AddEntry(
		FToolMenuEntry::InitMenuEntry("AllSettings",
			LOCTEXT("OpenSettingsText", "More Editor Performance Settings..."),
			LOCTEXT("OpenSettingsToolTip", "Open the Editor Performance Settings Tab."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([]()
				{
					FModuleManager::LoadModuleChecked<ISettingsModule>("Settings").ShowViewer("Editor", "General", "EditorPerformanceSettings");
				}),
				FCanExecuteAction()
			)));

	return SettingsMenu;

}

}

#undef LOCTEXT_NAMESPACE