// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ToolMenu.h"
#include "DataStorage/Handles.h"
#include "Widgets/SCompoundWidget.h"

namespace UE::Editor::DataStorage
{
	class SRowDetails;

	namespace QueryStack
	{
		class IRowNode;
	}

	class STedsSearchBox;
}

#define UE_API CRASHDIAGNOSTICS_API

namespace UE::Editor::CrashDiagnostics
{
	class SEditorCrashesPanel : public SCompoundWidget
	{
		static const FName SettingsMenuName;

	public:
		SLATE_BEGIN_ARGS(SEditorCrashesPanel) {}
		SLATE_END_ARGS()

		UE_API void Construct(const FArguments& InArgs);

	private:
		TSharedRef<SWidget> CreateTableWidget();
		TSharedRef<SWidget> CreateDetailsWidget();
		TSharedRef<SWidget> CreateSettingsMenuWidget();

		void OnSelectionChanged(UE::Editor::DataStorage::RowHandle RowHandle, ESelectInfo::Type SelectionType);

		UToolMenu* RegisterSettingsMenu();

	private:
		TSharedPtr<UE::Editor::DataStorage::QueryStack::IRowNode> SearchNode;

		TSharedPtr<UE::Editor::DataStorage::STedsSearchBox> SearchBox;
		TSharedPtr<UE::Editor::DataStorage::SRowDetails> CrashRowDetails;

		UE::Editor::DataStorage::RowHandle SelectedRowHandle = UE::Editor::DataStorage::InvalidRowHandle;
		float MarkAsReadDelaySeconds = 1.0f;
		TSharedPtr<FActiveTimerHandle> NewCrashTimer;
	};
}

#undef UE_API
