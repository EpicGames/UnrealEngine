// Copyright Epic Games, Inc. All Rights Reserved.

#include "SFileHyperlink.h"

#include "Misc/Paths.h"
#include "Widgets/Input/SHyperlink.h"


#define LOCTEXT_NAMESPACE "TedsCrashColumnWidgets"

namespace UE::Editor::CrashDiagnostics
{

void SFileHyperlink::Construct(const FArguments& InArgs, const FString& InPath)
{
	FullPath = FPaths::ConvertRelativePathToFull(InPath);
	OpenAction = InArgs._OpenAction;

	ChildSlot
		.HAlign(HAlign_Left)
		[
			SNew(SHyperlink)
			.Text(InArgs._TextOverride.IsSet()
				? InArgs._TextOverride
				: FText::FromString(FPaths::GetCleanFilename(FullPath)))
			.ToolTipText( this, &SFileHyperlink::GetToolTipText)
			.OnNavigate(this, &SFileHyperlink::Open)
		];
}

SFileHyperlink::EOpenAction SFileHyperlink::GetAction() const
{
	if (OpenAction == EOpenAction::None)
	{
		return EOpenAction::None;
	}

	EOpenAction Action = EOpenAction::None;
	if (FPaths::FileExists(FullPath))
	{
		if (OpenAction == EOpenAction::Auto || OpenAction == EOpenAction::OpenFileApplication)
		{
			Action = EOpenAction::OpenFileApplication;
		}
		else if (OpenAction == EOpenAction::OpenFolder)
		{
			Action = EOpenAction::OpenFolder;
		}
	}
	else if (FPaths::DirectoryExists(FullPath))
	{
		if (OpenAction == EOpenAction::Auto || OpenAction == EOpenAction::OpenFolder || OpenAction == EOpenAction::OpenFileApplication)
		{
			Action = EOpenAction::OpenFolder;
		}
	}

	return Action;
}

void SFileHyperlink::Open() const
{
	EOpenAction Action = GetAction();
	if (Action == EOpenAction::None)
	{
		return;
	}

	if (Action == EOpenAction::OpenFileApplication)
	{
		if (!FPlatformProcess::LaunchFileInDefaultExternalApplication(*FullPath))
		{
			Action = EOpenAction::OpenFolder;
		}
	}
	if (Action == EOpenAction::OpenFolder)
	{
		FPlatformProcess::ExploreFolder(*FullPath);
	}
}

FText SFileHyperlink::GetToolTipText() const
{
	const EOpenAction Action = GetAction();
	if (Action == EOpenAction::None)
	{
		return FText::FromString(FullPath);
	}

	return FText::Format(
		Action == EOpenAction::OpenFileApplication
		? LOCTEXT("OpenFileInDefaultApplication", "Open file in default application\n{0}")
		: LOCTEXT("OpenFolder", "Open folder\n{0}"),
		FText::FromString(FullPath)
	);
}

}

#undef LOCTEXT_NAMESPACE