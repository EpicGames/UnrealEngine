// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

namespace UE::Editor::CrashDiagnostics
{
	class SFileHyperlink : public SCompoundWidget
	{
	public:
		enum class EOpenAction : uint8
		{
			None,
			OpenFileApplication,
			OpenFolder,
			Auto,
		};

		SLATE_BEGIN_ARGS(SFileHyperlink)
			{}
			SLATE_ATTRIBUTE(FText, TextOverride)
			SLATE_ARGUMENT_DEFAULT(EOpenAction, OpenAction) { EOpenAction::Auto };
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, const FString& InPath);

	private:
		EOpenAction GetAction() const;

		void Open() const;

		FText GetToolTipText() const;

	private:
		FString FullPath;
		EOpenAction OpenAction = EOpenAction::Auto;
	};
}
