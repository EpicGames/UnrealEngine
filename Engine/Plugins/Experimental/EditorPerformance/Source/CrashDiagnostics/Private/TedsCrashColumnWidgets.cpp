// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsCrashColumnWidgets.h"

#include "CrashDiagnosticsModule.h"
#include "SFileHyperlink.h"
#include "TedsEditorCrashDataStorageFactory.h"
#include "Columns/SlateHeaderColumns.h"
#include "Columns/TedsCrashColumns.h"
#include "Elements/Framework/TypedElementAttributeBinding.h"
#include "Styling/StyleColors.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SEditableText.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"


#define LOCTEXT_NAMESPACE "TedsCrashColumnWidgets"

///////////////////////////////////
// FEditorCrashGUIDWidgetConstructor
///////////////////////////////////

TSharedPtr<SWidget> FEditorCrashGUIDWidgetConstructor::CreateWidget(
	UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle TargetRow,
	UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	UE::Editor::DataStorage::FAttributeBinder Binder(TargetRow, DataStorage);

	const FEditorCrashGUIDColumn* CrashGUID = DataStorage->GetColumn<FEditorCrashGUIDColumn>(TargetRow);
	const FEditorCrashFileReportsColumn* FileReports = DataStorage->GetColumn<FEditorCrashFileReportsColumn>(TargetRow);
	if (!ensure(CrashGUID) || !ensure(FileReports))
	{
		return SNullWidget::NullWidget;
	}

	using namespace UE::Editor::CrashDiagnostics;
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
		[
			SNew(SFileHyperlink, FileReports->GetCrashReportFolder())
				.TextOverride(Binder.BindText(&FEditorCrashGUIDColumn::CrashGUID))
				.OpenAction(SFileHyperlink::EOpenAction::OpenFolder)
		];
}


///////////////////////////////////
// FEditorCrashTimeWidgetConstructor
///////////////////////////////////

class FEditorCrashTimeWidgetSorter final : public UE::Editor::DataStorage::TColumnSorterInterface<UE::Editor::DataStorage::FColumnSorterInterface::ESortType::FixedSize64, FEditorCrashTimeColumn>
{
public:
	virtual FText GetShortName() const override
	{
		return LOCTEXT("FEditorCrashTimeWidgetSorter", "Editor Crash Time Sorter");
	}
protected:
	virtual UE::Editor::DataStorage::FPrefixInfo CalculatePrefix(const FEditorCrashTimeColumn& Column, uint32 ByteIndex) const override
	{
		return UE::Editor::DataStorage::CreateSortPrefix(ByteIndex, Column.TimeOfCrash.GetTicks());
	}
};


TSharedPtr<SWidget> FEditorCrashTimeWidgetConstructor::CreateWidget(
	UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle TargetRow,
	UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	UE::Editor::DataStorage::FAttributeBinder Binder(TargetRow, DataStorage);

	const FEditorCrashTimeColumn* CrashTime = DataStorage->GetColumn<FEditorCrashTimeColumn>(TargetRow);
	if (!ensure(CrashTime))
	{
		return SNullWidget::NullWidget;
	}

	TSharedPtr<SWidget> Widget;
	TAttribute<FText> TextAttribute = Binder.BindData(&FEditorCrashTimeColumn::TimeOfCrashString);

	const bool* bUseEditableText = Arguments.FindForColumnOrGeneric<FEditorCrashTimeColumn>("bUseEditableText").TryGetExact<bool>();
	if (bUseEditableText && (*bUseEditableText))
	{
		Widget = SNew(SEditableText)
			.IsReadOnly(true)
			.Text(MoveTemp(TextAttribute));
	}
	else
	{
		Widget = SNew(STextBlock)
			.Text(MoveTemp(TextAttribute));
	}

	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
		[
			Widget.ToSharedRef()
		];
}

TArray<TSharedPtr<const UE::Editor::DataStorage::FColumnSorterInterface>> FEditorCrashTimeWidgetConstructor::ConstructColumnSorters(
	UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;
	return TArray<TSharedPtr<const FColumnSorterInterface>>(
		{
			MakeShared<FEditorCrashTimeWidgetSorter>()
		});
}


///////////////////////////////////
// FEditorCrashTimeTableRowWidgetConstructor
///////////////////////////////////

TSharedPtr<SWidget> FEditorCrashTimeTableRowWidgetConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage, UE::Editor::DataStorage::IUiProvider* DataStorageUi, UE::Editor::DataStorage::RowHandle TargetRow, UE::Editor::DataStorage::RowHandle WidgetRow, const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	UE::Editor::DataStorage::FAttributeBinder Binder(TargetRow, DataStorage);

	const FEditorCrashTimeColumn* CrashTime = DataStorage->GetColumn<FEditorCrashTimeColumn>(TargetRow);
	const FEditorCrashGUIDColumn* CrashGUID = DataStorage->GetColumn<FEditorCrashGUIDColumn>(TargetRow);
	if (!ensure(CrashTime) || !ensure(CrashGUID))
	{
		return SNullWidget::NullWidget;
	}

	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
		[
			SNew(STextBlock)
			.Font_Lambda([DataStorage, TargetRow]()
			{
				const bool IsNewCrash = DataStorage->HasColumns(TargetRow, TArray{FEditorCrashIsNewTag::StaticStruct()});
				return IsNewCrash ?
					FAppStyle::Get().GetFontStyle("NormalFontBold") :
					FAppStyle::Get().GetFontStyle("NormalText");
			})
			.Text(Binder.BindData(&FEditorCrashTimeColumn::TimeOfCrashString, [DataStorage, TargetRow](const FText& InTimeOfCrash) mutable
			{
				const bool IsNewCrash = DataStorage->HasColumns(TargetRow, TArray{FEditorCrashIsNewTag::StaticStruct()});

				return FText::Format(LOCTEXT("NewFmt", "{0}{1}"),
					IsNewCrash ? INVTEXT("*") : FText::GetEmpty(),
					InTimeOfCrash
				);
			}))
			.ToolTipText(Binder.BindText(&FEditorCrashGUIDColumn::CrashGUID))
		];
}

TArray<TSharedPtr<const UE::Editor::DataStorage::FColumnSorterInterface>> FEditorCrashTimeTableRowWidgetConstructor::ConstructColumnSorters(UE::Editor::DataStorage::ICoreProvider* DataStorage, UE::Editor::DataStorage::IUiProvider* DataStorageUi, const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;
	return TArray<TSharedPtr<const FColumnSorterInterface>>(
		{
			MakeShared<FEditorCrashTimeWidgetSorter>()
		});
}


///////////////////////////////////
// FEditorCrashErrorMessageWidgetConstructor
///////////////////////////////////

TSharedPtr<SWidget> FEditorCrashErrorMessageWidgetConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage, UE::Editor::DataStorage::IUiProvider* DataStorageUi, UE::Editor::DataStorage::RowHandle TargetRow, UE::Editor::DataStorage::RowHandle WidgetRow, const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	UE::Editor::DataStorage::FAttributeBinder Binder(TargetRow, DataStorage);

	const FEditorCrashErrorMessageColumn* ErrorMessage = DataStorage->GetColumn<FEditorCrashErrorMessageColumn>(TargetRow);
	if (!ensure(ErrorMessage))
	{
		return SNullWidget::NullWidget;
	}

	return SNew(SMultiLineEditableTextBox)
		.IsReadOnly(true)
		.Style(UTedsEditorCrashDataStorageFactory::GetReadOnlyTextBoxStyle())
		.BackgroundColor(FStyleColors::Background)
		.Padding(FMargin(3.0f, 0.f))
		.Text(Binder.BindText(&FEditorCrashErrorMessageColumn::ErrorMessage));
}

TSharedPtr<SWidget> FEditorCrashErrorMessageCompactWidgetConstructor::CreateWidget(
	UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle TargetRow,
	UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	UE::Editor::DataStorage::FAttributeBinder Binder(TargetRow, DataStorage);

	const FEditorCrashErrorMessageColumn* ErrorMessage = DataStorage->GetColumn<FEditorCrashErrorMessageColumn>(TargetRow);
	if (!ensure(ErrorMessage))
	{
		return SNullWidget::NullWidget;
	}

	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.MaxWidth(0.f)
		.AutoWidth()
		[
			SNew(STextBlock) // The goal of this empty TextBlock is to give this HBox a height of 1 line of text
		]
		+ SHorizontalBox::Slot()
		.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
		.VAlign(VAlign_Fill)
		.HAlign(HAlign_Fill)
		.FillWidth(1.0f)
		[
			SNew(SBox)
			.HeightOverride(0.f) // This will make the Text below take only one line with ellipsis
			[
				SNew(STextBlock)
				.Text(Binder.BindText(&FEditorCrashErrorMessageColumn::ErrorMessage))
				.OverflowPolicy(ETextOverflowPolicy::MultilineEllipsis)
				.ToolTipText(Binder.BindText(&FEditorCrashErrorMessageColumn::ErrorMessage))
			]
		];
}


///////////////////////////////////
// FEditorCrashTypeWidgetConstructor
///////////////////////////////////

TSharedPtr<SWidget> FEditorCrashTypeWidgetConstructor::CreateWidget(
	UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle TargetRow,
	UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	UE::Editor::DataStorage::FAttributeBinder Binder(TargetRow, DataStorage);

	const FEditorCrashTypeColumn* CrashType = DataStorage->GetColumn<FEditorCrashTypeColumn>(TargetRow);
	if (!ensure(CrashType))
	{
		return SNullWidget::NullWidget;
	}

	TSharedPtr<SWidget> Widget;
	TAttribute<FText> TextAttribute = Binder.BindData(&FEditorCrashTypeColumn::CrashType, [](const FString& InCrashType) mutable
	{
		return FText::FromString(FName::NameToDisplayString(InCrashType, false));
	});

	const bool* bUseEditableText = Arguments.FindForColumnOrGeneric<FEditorCrashTypeColumn>("bUseEditableText").TryGetExact<bool>();
	if (bUseEditableText && (*bUseEditableText))
	{
		Widget = SNew(SEditableText)
			.IsReadOnly(true)
			.Text(MoveTemp(TextAttribute));
	}
	else
	{
		Widget = SNew(STextBlock)
			.Text(MoveTemp(TextAttribute));
	}

	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
		[
			Widget.ToSharedRef()
		];
}


///////////////////////////////////
// FEditorCrashFileReportsWidgetConstructor
///////////////////////////////////

TSharedPtr<SWidget> FEditorCrashFileReportsWidgetConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage, UE::Editor::DataStorage::IUiProvider* DataStorageUi, UE::Editor::DataStorage::RowHandle TargetRow, UE::Editor::DataStorage::RowHandle WidgetRow, const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::CrashDiagnostics;

	const FEditorCrashFileReportsColumn* FileReports = DataStorage->GetColumn<FEditorCrashFileReportsColumn>(TargetRow);
	if (!ensure(FileReports))
	{
		return SNullWidget::NullWidget;
	}

	TSharedRef<SVerticalBox> VerticalBox = SNew(SVerticalBox);
	for (const FString& ReportFileName : FileReports->ReportFilePaths)
	{
		VerticalBox->AddSlot()
		.Padding(0.f, 0.f, 0.f, 3.f)
		[
			SNew(SFileHyperlink, ReportFileName)
				.OpenAction(SFileHyperlink::EOpenAction::OpenFileApplication)
		];
	}

	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
		[
			MoveTemp(VerticalBox)
		];
}


///////////////////////////////////
// FEditorCrashCallStackWidgetConstructor
///////////////////////////////////

TSharedPtr<SWidget> FEditorCrashCallStackWidgetConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage, UE::Editor::DataStorage::IUiProvider* DataStorageUi, UE::Editor::DataStorage::RowHandle TargetRow, UE::Editor::DataStorage::RowHandle WidgetRow, const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	UE::Editor::DataStorage::FAttributeBinder Binder(TargetRow, DataStorage);

	const FEditorCrashCallStackColumn* CallStack = DataStorage->GetColumn<FEditorCrashCallStackColumn>(TargetRow);
	if (!ensure(CallStack))
	{
		return SNullWidget::NullWidget;
	}

	return SNew(SMultiLineEditableTextBox)
		.IsReadOnly(true)
		.Style(UTedsEditorCrashDataStorageFactory::GetReadOnlyTextBoxStyle())
		.BackgroundColor(FStyleColors::Background)
		.Padding(FMargin(3.0f, 0.f))
		.Text(Binder.BindText(&FEditorCrashCallStackColumn::CallStack));
}


///////////////////////////////////
// FEditorCrashSourceContextWidgetConstructor
///////////////////////////////////

TSharedPtr<SWidget> FEditorCrashSourceContextWidgetConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage, UE::Editor::DataStorage::IUiProvider* DataStorageUi, UE::Editor::DataStorage::RowHandle TargetRow, UE::Editor::DataStorage::RowHandle WidgetRow, const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	UE::Editor::DataStorage::FAttributeBinder Binder(TargetRow, DataStorage);

	const FEditorCrashSourceContextColumn* SourceContext = DataStorage->GetColumn<FEditorCrashSourceContextColumn>(TargetRow);
	if (!ensure(SourceContext))
	{
		return SNullWidget::NullWidget;
	}

	return SNew(SMultiLineEditableTextBox)
		.IsReadOnly(true)
		.Style(UTedsEditorCrashDataStorageFactory::GetReadOnlyTextBoxStyle())
		.BackgroundColor(FStyleColors::Background)
		.Padding(FMargin(2.0f, 0.f))
		.Text(Binder.BindText(&FEditorCrashSourceContextColumn::SourceContext));
}


///////////////////////////////////
// FEditorCrashUserActivityHintWidgetConstructor
///////////////////////////////////

TSharedPtr<SWidget> FEditorCrashUserActivityHintWidgetConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage, UE::Editor::DataStorage::IUiProvider* DataStorageUi, UE::Editor::DataStorage::RowHandle TargetRow, UE::Editor::DataStorage::RowHandle WidgetRow, const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	UE::Editor::DataStorage::FAttributeBinder Binder(TargetRow, DataStorage);

	const FEditorCrashUserActivityHintColumn* ActivityHint = DataStorage->GetColumn<FEditorCrashUserActivityHintColumn>(TargetRow);
	if (!ensure(ActivityHint))
	{
		return SNullWidget::NullWidget;
	}

	TSharedPtr<SWidget> Widget;
	TAttribute<FText> TextAttribute = Binder.BindText(&FEditorCrashUserActivityHintColumn::UserActivityHint);

	const bool* bUseEditableText = Arguments.FindForColumnOrGeneric<FEditorCrashUserActivityHintColumn>("bUseEditableText").TryGetExact<bool>();
	if (bUseEditableText && (*bUseEditableText))
	{
		Widget = SNew(SEditableText)
			.IsReadOnly(true)
			.Text(MoveTemp(TextAttribute));
	}
	else
	{
		Widget = SNew(STextBlock)
			.Text(MoveTemp(TextAttribute));
	}

	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
		[
			Widget.ToSharedRef()
		];
}


///////////////////////////////////
// FEditorCrashTimeHeaderWidgetConstructor
///////////////////////////////////

TSharedPtr<SWidget> FEditorCrashTimeHeaderWidgetConstructor::CreateWidget(const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	return SNew(STextBlock)
		.Text(FEditorCrashTimeColumn::StaticStruct()->GetDisplayNameText());
}

bool FEditorCrashTimeHeaderWidgetConstructor::FinalizeWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage, UE::Editor::DataStorage::IUiProvider* DataStorageUi, UE::Editor::DataStorage::RowHandle Row, const TSharedPtr<SWidget>& Widget)
{
	DataStorage->AddColumn(Row, FHeaderWidgetSizeColumn
		{
			.ColumnSizeMode = EColumnSizeMode::Manual,
			.Width = 200.0f
		});
	return true;
}


///////////////////////////////////
// FEditorCrashTypeHeaderWidgetConstructor
///////////////////////////////////

TSharedPtr<SWidget> FEditorCrashTypeHeaderWidgetConstructor::CreateWidget(const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	return SNew(STextBlock)
		.Text(FEditorCrashTypeColumn::StaticStruct()->GetDisplayNameText());
}

bool FEditorCrashTypeHeaderWidgetConstructor::FinalizeWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage, UE::Editor::DataStorage::IUiProvider* DataStorageUi, UE::Editor::DataStorage::RowHandle Row, const TSharedPtr<SWidget>& Widget)
{
	DataStorage->AddColumn(Row, FHeaderWidgetSizeColumn
		{
			.ColumnSizeMode = EColumnSizeMode::Manual,
			.Width = 150.0f
		});
	return true;
}

#undef LOCTEXT_NAMESPACE
