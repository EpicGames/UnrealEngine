// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"

#include "TedsCrashColumnWidgets.generated.h"


///////////////////////////////////
// Widget Constructors
///////////////////////////////////

USTRUCT()
struct FEditorCrashGUIDWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	FEditorCrashGUIDWidgetConstructor()
		: Super(StaticStruct())
	{
	}

	virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
};


USTRUCT()
struct FEditorCrashTimeWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	FEditorCrashTimeWidgetConstructor()
		: Super(StaticStruct())
	{
	}

	virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;

	virtual TArray<TSharedPtr<const UE::Editor::DataStorage::FColumnSorterInterface>> ConstructColumnSorters(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
};

USTRUCT()
struct FEditorCrashTimeTableRowWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	FEditorCrashTimeTableRowWidgetConstructor()
		: Super(StaticStruct())
	{
	}

	virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;

	virtual TArray<TSharedPtr<const UE::Editor::DataStorage::FColumnSorterInterface>> ConstructColumnSorters(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
};


USTRUCT()
struct FEditorCrashErrorMessageWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	FEditorCrashErrorMessageWidgetConstructor()
		: Super(StaticStruct())
	{
	}

	virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
};

USTRUCT()
struct FEditorCrashErrorMessageCompactWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	FEditorCrashErrorMessageCompactWidgetConstructor()
		: Super(StaticStruct())
	{
	}

	virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
};


USTRUCT()
struct FEditorCrashTypeWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	FEditorCrashTypeWidgetConstructor()
		: Super(StaticStruct())
	{
	}

	virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
};


USTRUCT()
struct FEditorCrashFileReportsWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	FEditorCrashFileReportsWidgetConstructor()
		: Super(StaticStruct())
	{
	}

	virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
};


USTRUCT()
struct FEditorCrashCallStackWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	FEditorCrashCallStackWidgetConstructor()
		: Super(StaticStruct())
	{
	}

	virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
};


USTRUCT()
struct FEditorCrashSourceContextWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	FEditorCrashSourceContextWidgetConstructor()
		: Super(StaticStruct())
	{
	}

	virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
};


USTRUCT()
struct FEditorCrashUserActivityHintWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	FEditorCrashUserActivityHintWidgetConstructor()
		: Super(StaticStruct())
	{
	}

	virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
};


///////////////////////////////////
// Header Widget Constructors
///////////////////////////////////

USTRUCT()
struct FEditorCrashTimeHeaderWidgetConstructor final : public FTypedElementWidgetConstructor
{
	GENERATED_BODY()

public:
	FEditorCrashTimeHeaderWidgetConstructor()
		: Super(StaticStruct())
	{
	}

protected:
	virtual TSharedPtr<SWidget> CreateWidget(const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
	virtual bool FinalizeWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle Row,
		const TSharedPtr<SWidget>& Widget) override;
};


USTRUCT()
struct FEditorCrashTypeHeaderWidgetConstructor final : public FTypedElementWidgetConstructor
{
	GENERATED_BODY()

public:
	FEditorCrashTypeHeaderWidgetConstructor()
		: Super(StaticStruct())
	{
	}

protected:
	virtual TSharedPtr<SWidget> CreateWidget(const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
	virtual bool FinalizeWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle Row,
		const TSharedPtr<SWidget>& Widget) override;
};