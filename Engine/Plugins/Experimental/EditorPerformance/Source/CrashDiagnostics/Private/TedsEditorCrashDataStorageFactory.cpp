// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsEditorCrashDataStorageFactory.h"

#include "CrashDiagnosticsModule.h"
#include "TedsCrashColumnWidgets.h"
#include "Brushes/SlateNoResource.h"
#include "Columns/TedsCrashColumns.h"
#include "DataStorage/Features.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateTypes.h"

#define LOCTEXT_NAMESPACE "TedsEditorCrashDataStorageFactory"

const UE::Editor::DataStorage::IUiProvider::FPurposeInfo& UTedsEditorCrashDataStorageFactory::GetTableRowPurpose()
{
	using namespace UE::Editor::DataStorage;
	static IUiProvider::FPurposeInfo Purpose(
		"CrashDiagnostics", "Cell", "TableRow",
		IUiProvider::EPurposeType::UniqueByNameAndColumn,
		LOCTEXT("CompactPurposeDescription", "Purpose to display widgets compactly in a table. (ex: show only one line for multiline texts)"),
		GetGeneralPurposeID());
	return Purpose;
}

UE::Editor::DataStorage::IUiProvider::FPurposeID UTedsEditorCrashDataStorageFactory::GetGeneralPurposeID()
{
	using namespace UE::Editor::DataStorage;
	IUiProvider* DataStorageUi = GetMutableDataStorageFeature<IUiProvider>(UiFeatureName);
	check(DataStorageUi)
	return DataStorageUi->GetGeneralWidgetPurposeID();
}

UE::Editor::DataStorage::IUiProvider::FPurposeID UTedsEditorCrashDataStorageFactory::GetTableRowPurposeID()
{
	return GetTableRowPurpose().GeneratePurposeID();
}

UE::Editor::DataStorage::IUiProvider::FPurposeID UTedsEditorCrashDataStorageFactory::GetHeaderRowPurposeID()
{
	using namespace UE::Editor::DataStorage;
	static IUiProvider::FPurposeInfo HeaderPurpose("General", "Header", NAME_None);
	return HeaderPurpose.GeneratePurposeID();
}

FEditableTextBoxStyle* UTedsEditorCrashDataStorageFactory::GetReadOnlyTextBoxStyle()
{
	static FEditableTextBoxStyle Style = FEditableTextBoxStyle(FCoreStyle::Get().GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox"))
		.SetBackgroundImageReadOnly(FSlateNoResource());
	return &Style;
}

void UTedsEditorCrashDataStorageFactory::RegisterTables(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	DataStorage.RegisterTable<
		FEditorCrashGUIDColumn,
		FEditorCrashTimeColumn,
		FEditorCrashTypeColumn,
		FEditorCrashFileReportsColumn
	>(UE::Editor::CrashDiagnostics::DataTableName);

	DataStorage.RegisterTable(UE::Editor::CrashDiagnostics::GlobalTableName);
}

void UTedsEditorCrashDataStorageFactory::RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::Queries;

	HasCrashedLastSessionQueryHandle = DataStorage.RegisterQuery(
		Count()
		.Where()
			.Any<FEditorCrashLastSessionTag>()
		.Compile());

	HasUnreadCrashesQueryHandle = DataStorage.RegisterQuery(
		Count()
		.Where()
			.Any<FEditorCrashIsNewTag>()
		.Compile());
}

void UTedsEditorCrashDataStorageFactory::RegisterWidgetPurposes(UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	DataStorageUi.RegisterWidgetPurpose(GetTableRowPurpose());
}

void UTedsEditorCrashDataStorageFactory::RegisterWidgetConstructors(
	UE::Editor::DataStorage::ICoreProvider& DataStorage,
	UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;

	///////////////////////////////////
	// Widget Constructors
	///////////////////////////////////

	{
		const IUiProvider::FPurposeID PurposeID = GetGeneralPurposeID();

		DataStorageUi.RegisterWidgetFactory<FEditorCrashGUIDWidgetConstructor>(
			DataStorageUi.FindPurpose(PurposeID),
			TColumn<FEditorCrashGUIDColumn>()
		);

		DataStorageUi.RegisterWidgetFactory<FEditorCrashFileReportsWidgetConstructor>(
			DataStorageUi.FindPurpose(PurposeID),
			TColumn<FEditorCrashFileReportsColumn>()
		);

		DataStorageUi.RegisterWidgetFactory<FEditorCrashTimeWidgetConstructor>(
			DataStorageUi.FindPurpose(PurposeID),
			TColumn<FEditorCrashTimeColumn>()
		);

		DataStorageUi.RegisterWidgetFactory<FEditorCrashErrorMessageWidgetConstructor>(
			DataStorageUi.FindPurpose(PurposeID),
			TColumn<FEditorCrashErrorMessageColumn>()
		);

		DataStorageUi.RegisterWidgetFactory<FEditorCrashTypeWidgetConstructor>(
			DataStorageUi.FindPurpose(PurposeID),
			TColumn<FEditorCrashTypeColumn>()
		);

		DataStorageUi.RegisterWidgetFactory<FEditorCrashCallStackWidgetConstructor>(
			DataStorageUi.FindPurpose(PurposeID),
			TColumn<FEditorCrashCallStackColumn>()
		);

		DataStorageUi.RegisterWidgetFactory<FEditorCrashSourceContextWidgetConstructor>(
			DataStorageUi.FindPurpose(PurposeID),
			TColumn<FEditorCrashSourceContextColumn>()
		);

		DataStorageUi.RegisterWidgetFactory<FEditorCrashUserActivityHintWidgetConstructor>(
			DataStorageUi.FindPurpose(PurposeID),
			TColumn<FEditorCrashUserActivityHintColumn>()
		);
	}

	///////////////////////////////////
	// Compact Widget Constructors
	///////////////////////////////////

	{
		const IUiProvider::FPurposeID PurposeID = GetTableRowPurposeID();

		DataStorageUi.RegisterWidgetFactory<FEditorCrashTimeTableRowWidgetConstructor>(
			DataStorageUi.FindPurpose(PurposeID),
			TColumn<FEditorCrashTimeColumn>()
		);

		DataStorageUi.RegisterWidgetFactory<FEditorCrashErrorMessageCompactWidgetConstructor>(
			DataStorageUi.FindPurpose(PurposeID),
			TColumn<FEditorCrashErrorMessageColumn>()
		);
	}

	///////////////////////////////////
	// Header Widget Constructors
	///////////////////////////////////

	{
		const IUiProvider::FPurposeID PurposeID = GetHeaderRowPurposeID();

		DataStorageUi.RegisterWidgetFactory<FEditorCrashTimeHeaderWidgetConstructor>(
			DataStorageUi.FindPurpose(PurposeID),
			TColumn<FEditorCrashTimeColumn>());

		DataStorageUi.RegisterWidgetFactory<FEditorCrashTypeHeaderWidgetConstructor>(
			DataStorageUi.FindPurpose(PurposeID),
			TColumn<FEditorCrashTypeColumn>());
	}
}

#undef LOCTEXT_NAMESPACE