// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"

#include "TedsEditorCrashDataStorageFactory.generated.h"

struct FEditableTextBoxStyle;

UCLASS()
class UTedsEditorCrashDataStorageFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UTedsEditorCrashDataStorageFactory() override = default;

	static const UE::Editor::DataStorage::IUiProvider::FPurposeInfo& GetTableRowPurpose();
	static UE::Editor::DataStorage::IUiProvider::FPurposeID GetGeneralPurposeID();
	static UE::Editor::DataStorage::IUiProvider::FPurposeID GetTableRowPurposeID();
	static UE::Editor::DataStorage::IUiProvider::FPurposeID GetHeaderRowPurposeID();

	static FEditableTextBoxStyle* GetReadOnlyTextBoxStyle();

	virtual void RegisterTables(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
	virtual void RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
	virtual void RegisterWidgetPurposes(UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
	virtual void RegisterWidgetConstructors(
		UE::Editor::DataStorage::ICoreProvider& DataStorage,
		UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;

	UE::Editor::DataStorage::QueryHandle GetHasCrashedLastSessionQueryHandle() const
	{
		return HasCrashedLastSessionQueryHandle;
	}
	UE::Editor::DataStorage::QueryHandle GetHasUnreadCrashesQueryHandle() const
	{
		return HasUnreadCrashesQueryHandle;
	}
private:
	UE::Editor::DataStorage::QueryHandle HasCrashedLastSessionQueryHandle = UE::Editor::DataStorage::InvalidQueryHandle;
	UE::Editor::DataStorage::QueryHandle HasUnreadCrashesQueryHandle = UE::Editor::DataStorage::InvalidQueryHandle;
};
