// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/CommonTypes.h"
#include "Misc/DateTime.h"
#include "Misc/Paths.h"

#include "TedsCrashColumns.generated.h"

namespace UE::Editor::CrashDiagnostics
{
	inline static const FName MappingDomain = "CrashDiagnostics";
	inline static const FName DataTableName = "CrashDiagnosticsDataTable";
	inline static const FName GlobalTableName = "CrashDiagnosticsGlobalTable";
	inline static const FName LastSessionCrashStateRowKey = "CrashDiagnosticsLastSessionCrashStateRowKey";
}

USTRUCT(DisplayName="File Reports")
struct FEditorCrashFileReportsColumn : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY(meta = (Searchable))
	FString XmlFilePath;

	UPROPERTY(meta = (Searchable))
	FString LogFilePath;

	UPROPERTY()
	TArray<FString> ReportFilePaths;

	FString GetCrashReportFolder() const
	{
		return FPaths::GetPath(XmlFilePath);
	}
};

USTRUCT(DisplayName="Time of Crash")
struct FEditorCrashTimeColumn : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY()
	FDateTime TimeOfCrash;

	UPROPERTY(meta = (Searchable))
	FText TimeOfCrashString;
};

USTRUCT(DisplayName="Crash ID")
struct FEditorCrashGUIDColumn : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY(meta = (Searchable, Sortable))
	FString CrashGUID;
};

USTRUCT(DisplayName="Error Message")
struct FEditorCrashErrorMessageColumn : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY(meta = (Searchable))
	FString ErrorMessage;
};

USTRUCT(DisplayName="Call Stack")
struct FEditorCrashCallStackColumn : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY(meta = (Searchable))
	FString CallStack;
};

USTRUCT(DisplayName="Source Context")
struct FEditorCrashSourceContextColumn : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY(meta = (Searchable))
	FString SourceContext;
};

USTRUCT(DisplayName="Crash Type")
struct FEditorCrashTypeColumn : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY(meta = (Searchable, Sortable))
	FString CrashType;
};

USTRUCT(DisplayName="User Activity")
struct FEditorCrashUserActivityHintColumn : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY(meta = (Searchable, Sortable))
	FString UserActivityHint;
};

USTRUCT(DisplayName="Is Ensure")
struct FEditorCrashIsEnsureTag : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

USTRUCT(DisplayName="Is Out Of Memory")
struct FEditorCrashIsOOMTag : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

USTRUCT(DisplayName="Is New Crash")
struct FEditorCrashIsNewTag : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

USTRUCT(DisplayName="Crashed Last Session")
struct FEditorCrashLastSessionTag : public FEditorDataStorageTag
{
	GENERATED_BODY()
};
