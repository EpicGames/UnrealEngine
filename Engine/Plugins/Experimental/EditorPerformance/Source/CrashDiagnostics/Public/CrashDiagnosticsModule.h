// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/DateTime.h"
#include "Modules/ModuleManager.h"

#define UE_API CRASHDIAGNOSTICS_API

struct FPrimaryCrashProperties;
struct FSlateBrush;
class SWidget;
namespace UE::Editor::DataStorage
{
	class ICoreProvider;
}

namespace UE::Editor::CrashDiagnostics
{

class FCrashDiagnosticsModule : public IModuleInterface
{
public:
	static FCrashDiagnosticsModule& GetChecked()
	{
		return FModuleManager::GetModuleChecked<FCrashDiagnosticsModule>("CrashDiagnostics");
	}
	static FCrashDiagnosticsModule* Get()
	{
		return FModuleManager::GetModulePtr<FCrashDiagnosticsModule>("CrashDiagnostics");
	}

	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	/**
	 * Retrieve the crash data from the Saved directory
	 */
	static TArray<TSharedRef<FPrimaryCrashProperties>> RetrieveSavedCrashes();

	/**
	 * Add the crashes to the given DataStorage. See TedsCrashColumns.h
	 */
	void AddCrashesToDataStorage(UE::Editor::DataStorage::ICoreProvider& DataStorage);
	void AddCrashesToDataStorage(UE::Editor::DataStorage::ICoreProvider& DataStorage, const TArray<TSharedRef<FPrimaryCrashProperties>>& Crashes);

	/**
	 * Add the crashes asynchronously to the default DataStorage.
	 * This function will call RetrieveSavedCrashes on a background thread before calling AddCrashesToDataStorage on the GT.
	 */
	void AddCrashesToDataStorageAsync();

	void LoadCrashDiagnosticsSettings();
	void SaveCrashDiagnosticsSettings();

	bool HasCrashedLastSession() const;
	bool HasUnreadCrashes() const;

	UE_API TSharedRef<SWidget> CreateCrashLogPanel();
	/**
	 * Function used for a GetDecoratedButtonDelegate that will add a badge to the first found SLayeredImage.
	 * This badge will only show if there are unread crashes.
	 */
	static TSharedRef<SWidget> DecorateButtonWithUnreadBadge(TSharedRef<SWidget> InWidget);
	const FSlateBrush* GetUnreadBadgeIcon() const;

private:
	void OnEditorInitialized(double TimeToInitializeEditor);

	FDateTime PreviousSessionStartTime = FDateTime();
};

}

#undef UE_API
