// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class EditorPerformance : ModuleRules
{
	public EditorPerformance(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"CrashDiagnostics",
				"Engine",
				"Slate",
				"ApplicationCore",
				"SlateCore",
				"InputCore",
				"EditorFramework",
				"EditorWidgets",
				"UnrealEd",
				"ToolMenus",
				"OutputLog",
				"EditorSubsystem",
				"WorkspaceMenuStructure",
				"MessageLog",
				"ToolWidgets",
				"DerivedDataCache",
				"Virtualization",
				"StudioTelemetry",
				"StallLogSubsystem",
				"TypedElementFramework"
			});

		PrivateIncludePathModuleNames.AddRange
		(
			new string[]
			{
				"WorkspaceMenuStructure",
			}
		);

		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
	}
}
