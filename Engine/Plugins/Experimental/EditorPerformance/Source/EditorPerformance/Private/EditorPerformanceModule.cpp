// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorPerformanceModule.h"

#include "Framework/Application/SlateApplication.h"
#include "Modules/ModuleManager.h"
#include "SEditorDiagnosticsTab.h"
#include "SEditorPerformanceDialogs.h"
#include "SEditorPerformanceStatusBar.h"
#include "Styling/AppStyle.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Docking/SDockTab.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

#include "KPIValue.h"
#include "Editor.h"
#include "ToolMenus.h"
#include "Editor/EditorPerformanceSettings.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "DerivedDataCacheUsageStats.h"
#include "Virtualization/VirtualizationSystem.h"
#include "Trace/Trace.h"
#include "StudioTelemetry.h"
#include "HAL/PlatformFileManager.h"
#include "ProfilingDebugging/StallDetector.h"
#include "DataStorage/Features.h"
#include "Diagnostics/EditorDiagnosticsColumns.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "TedsEditorPerformanceFactory.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"

#define LOCTEXT_NAMESPACE "EditorPerformance"
 
IMPLEMENT_MODULE(FEditorPerformanceModule, EditorPerformance );

// Declare the KPI names and limits
const FName EditorCategoryName = TEXT("Editor");
const FText EditorCategoryText = LOCTEXT("Editor", "Editor");

const FName PIECategoryName = TEXT("PIE");
const FText PIECategoryText = LOCTEXT("PIE", "PIE");

const FName CacheCategoryName = TEXT("Cache");
const FText CacheCategoryText = LOCTEXT("Cache", "Cache");

const FName MemoryCategoryName = TEXT("Memory");
const FText MemoryCategoryText = LOCTEXT("Memory", "Memory");

const FName DiskCategoryName = TEXT("Disk");
const FText DiskCategoryText = LOCTEXT("Disk", "Disk");

const FName EditorBootKPIName = TEXT("Boot");
const FText EditorBootKPIText = LOCTEXT("EditorBoot", "Boot");

const FName EditorInitializeKPIName = TEXT("Initialize");
const FText EditorInitializeKPIText = LOCTEXT("EditorInitialize", "Initialize");

const FName EditorLoadMapKPIName = TEXT("Load Map");
const FText EditorLoadMapKPIText = LOCTEXT("EditorLoadMap", "Load Map");

const FName EditorHitchRateKPIName = TEXT("Hitch Rate");
const FText EditorHitchRateKPIText = LOCTEXT("EditorHitchRate", "Hitch Rate");

const FName EditorStallRateKPIName = TEXT("Stall Rate");
const FText EditorStallRateKPIText = LOCTEXT("EditorStallRate", "Stall Rate");

const FName EditorAssetRegistryScanKPIName = TEXT("Asset Registry Scan");
const FText EditorAssetRegistryScanKPIText = LOCTEXT("EditorAssetRegistryScan", "Asset Registry Scan");

const FName EditorPluginCountKPIName = TEXT("Plugin Count");
const FText EditorPluginCountKPIText = LOCTEXT("EditorPluginCount", "Plugin Count");

const FName TotalTimeToEditorKPIName = TEXT("Total Time To Editor");
const FText TotalTimeToEditorKPIText = LOCTEXT("TotalTimeToEditor", "Total Time To Editor");

const FName TotalTimeToPIEKPIName = TEXT("Total Time To PIE");
const FText TotalTimeToPIEKPIText = LOCTEXT("TotalTimeToPIE", "Total Time To PIE");

const FName PIEFirstTransitionKPIName = TEXT("First Transition");
const FText PIEFirstTransitionKPIText = LOCTEXT("PIEFirstTransition", "First Transition");

const FName PIETransitionKPIName = TEXT("Iterative Transition");
const FText PIETransitionKPIText = LOCTEXT("PIEIterativeTransition", "Iterative Transition");

const FName PIEShutdownKPIName = TEXT("Shutdown");
const FText PIEShutdownKPIText = LOCTEXT("PIEShutdown", "Shutdown");

const FName PIEHitchRateKPIName = TEXT("Hitch Rate");
const FText PIEHitchRateKPIText = LOCTEXT("PIEHitchRate", "Hitch Rate");

const FName PIEStallRateKPIName = TEXT("Stall Rate");
const FText PIEStallRateKPIText = LOCTEXT("PIEStallRate", "Stall Rate");

const FName CloudDDCLatencyKPIName = TEXT("Unreal Cloud DDC Latency");
const FText CloudDDCLatencyKPIText = LOCTEXT("CloudDDCLatency", "Unreal Cloud Latency");

const FName CloudDDCReadSpeedKPIName = TEXT("Unreal Cloud DDC Speed");
const FText CloudDDCReadSpeedKPIText = LOCTEXT("CloudDDCReadSpeed", "Unreal Cloud DDC Speed");

const FName LocalDDCEfficiencyKPIName = TEXT("Local Efficiency");
const FText LocalDDCEfficiencyKPIText = LOCTEXT("LocalDDCEfficiency", "Local Efficiency");

const FName VirtualAssetEfficiencyKPIName = TEXT("Virtual Asset Efficiency");
const FText VirtualAssetEfficiencyKPIText = LOCTEXT("VirtualAssetEfficiency", "Virtual Asset Efficiency");

const FName AvailableMemoryKPIName = TEXT("Available Memory");
const FText AvailableMemoryKPIText = LOCTEXT("AvailableMemory", "Available Memory");

const FName MemoryPressureKPIName = TEXT("Memory Pressure");
const FText MemoryPressureKPIText = LOCTEXT("MemoryPressure", "Memory Pressure");

const FName DynamicInvalidationKPIName = TEXT("Dynamic Invalidation");
const FText DynamicInvalidationKPIText = LOCTEXT("DynamicInvalidation", "Dynamic Invalidation");

const FName EngineDirFreeSpaceKPIName = TEXT("Engine Dir Free Space");
const FText EngineDirFreeSpaceKPIText = LOCTEXT("EngineDirFreeSpace", "Engine Dir Free Space");

const FName ProjectDirFreeSpaceKPIName = TEXT("Project Dir Free Space");
const FText ProjectDirFreeSpaceKPIText = LOCTEXT("ProjectDirFreeSpace", "Project Dir Free Space");

const FName UserDirFreeSpaceKPIName = TEXT("User Dir Free Space");
const FText UserDirFreeSpaceKPIText = LOCTEXT("UserDirFreeSpace", "User Dir Free Space");

// Thresholds
TOptional<float> NoSetLimit;
float LocalDDCEfficiencyKPILimit = 85.f;
float VirtualAssetEfficiencyKPILimit = 95.f;
float MemoryPressureKPILimit = (float)FGenericPlatformMemoryStats::EMemoryPressureStatus::Warning;
float DynamicInvalidationKPILimit = 1.0f;
 
static const FName EditorDiagnosticsTabName = FName(TEXT("EditorDiagnosticsTab"));

static FString GForceBadKPIState = TEXT("");
static FAutoConsoleVariableRef CVarForceBadKPIState(
	TEXT("Editor.PerformanceKPI.ForceBadState"),
	GForceBadKPIState,
	TEXT("The name (category_name) of a KPI to force into a bad state. The KPI will stay in a bad state as long as this cvar isn't cleared."),
	ECVF_Default);

void FEditorPerformanceModule::StartupModule()
{
	InitializeKPIs();
}

void FEditorPerformanceModule::ShutdownModule()
{
	TerminateEditor();
	TerminateKPIs();
}

void FEditorPerformanceModule::InitializeEditor()
{
	UEditorPerformanceSettings* EditorPerformanceSettings = GetMutableDefault<UEditorPerformanceSettings>();

	constexpr bool bLoop = true;

	FTimerDelegate HighFrequencyUpdateDelegate;
	HighFrequencyUpdateDelegate.BindRaw(this, &FEditorPerformanceModule::UpdateKPIsHighFrequency);
	GEditor->GetTimerManager()->SetTimer(HighFrequencyUpdateTimerHandle, HighFrequencyUpdateDelegate, HighFrequencyUpdateIntervalSeconds, bLoop);
	
	FTimerDelegate LowFrequencyUpdateDelegate;
	LowFrequencyUpdateDelegate.BindRaw(this, &FEditorPerformanceModule::UpdateKPIsLowFrequency);
	GEditor->GetTimerManager()->SetTimer(LowFrequencyUpdateTimerHandle, LowFrequencyUpdateDelegate, LowFrequencyUpdateIntervalSeconds, bLoop);

	FTimerDelegate HitchSamplerDelegate;
	HitchSamplerDelegate.BindRaw(this, &FEditorPerformanceModule::HitchSamplerCallback);
	GEditor->GetTimerManager()->SetTimer(HitchSamplerTimerHandle, HitchSamplerDelegate, HitchSamplerIntervalSeconds, true);

	const FSlateIcon PerformanceReportIcon(FAppStyle::GetAppStyleSetName(), "EditorPerformance.Report.Panel");

	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(EditorDiagnosticsTabName, FOnSpawnTab::CreateRaw(this, &FEditorPerformanceModule::CreatePerformanceReportTab))
		.SetDisplayName(LOCTEXT("EditorDiagnosticsTabTitle", "Editor Diagnostics"))
		.SetTooltipText(LOCTEXT("EditorDiagnosticsTabToolTipText", "Opens the Editor Diagnostics tab."))
		.SetGroup(WorkspaceMenu::GetMenuStructure().GetDeveloperToolsProfilingCategory())
		.SetIcon(PerformanceReportIcon);

#if WITH_RELOAD
	// This code attempts to relaunch the tabs when you reload this module
	if (IsReloadActive() && FSlateApplication::IsInitialized())
	{
		ShowPerformanceReportTab();
	}
#endif // WITH_RELOAD

	FEditorDiagnosticsCommands::Register();

	UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("LevelEditor.StatusBar.ToolBar");

	if (Menu != nullptr)
	{
		// Add the Editor Perf ToolBar 
		FToolMenuSection& EditorPerfSection = Menu->AddSection("EditorPerf", FText::GetEmpty(), FToolMenuInsert("Insights", EToolMenuInsertType::Before));
		EditorPerfSection.AddEntry(FToolMenuEntry::InitWidget("EditorPerformanceStatusBar", CreateStatusBarWidget(), FText::GetEmpty(), true, false));
	}
	
	// Enable any experimental features
	if (EditorPerformanceSettings && EditorPerformanceSettings->bEnableExperimentalFeatures)
	{
		CloudDDCLatencyKPI			= KPIRegistry.DeclareKPIValue(CacheCategoryName, CacheCategoryText, CloudDDCLatencyKPIName, CloudDDCLatencyKPIText, 0.0, NoSetLimit, FKPIValue::LessThan, FKPIValue::Milliseconds);
		CloudDDCReadSpeedKPI		= KPIRegistry.DeclareKPIValue(CacheCategoryName, CacheCategoryText, CloudDDCReadSpeedKPIName, CloudDDCReadSpeedKPIText, 100.0, NoSetLimit, FKPIValue::GreaterThan, FKPIValue::MegaBitsPerSecond);
		EditorStallRateKPI			= KPIRegistry.DeclareKPIValue(EditorCategoryName, EditorCategoryText, EditorStallRateKPIName, EditorStallRateKPIText, 0.0, NoSetLimit, FKPIValue::LessThan, FKPIValue::Percent);
		PIEStallRateKPI				= KPIRegistry.DeclareKPIValue(PIECategoryName, PIECategoryText, PIEStallRateKPIName, PIEStallRateKPIText, 0.0, NoSetLimit, FKPIValue::LessThan, FKPIValue::Percent);
	}

	// Update low frequency KPIs right away so that we have initial values for them without having to way for the timer to tick once
	UpdateKPIsLowFrequency();
}

void FEditorPerformanceModule::TerminateEditor()
{
	if (FSlateApplication::IsInitialized())
	{
		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(EditorDiagnosticsTabName);

		if (PerformanceReportTab.IsValid())
		{
			PerformanceReportTab.Pin()->RequestCloseTab();
		}
	}

	if (GEditor)
	{
		GEditor->GetTimerManager()->ClearTimer(HighFrequencyUpdateTimerHandle);
		GEditor->GetTimerManager()->ClearTimer(LowFrequencyUpdateTimerHandle);
		GEditor->GetTimerManager()->ClearTimer(HitchSamplerTimerHandle);
	}

	FEditorDiagnosticsCommands::Unregister();
}

TSharedRef<SWidget> FEditorPerformanceModule::CreateStatusBarWidget()
{
	return SNew(SEditorPerformanceStatusBarWidget);
}

TSharedRef<SDockTab> FEditorPerformanceModule::CreatePerformanceReportTab(const FSpawnTabArgs& Args)
{
	return SNew(SEditorDiagnosticsTab).TabRole(ETabRole::NomadTab);
}

void FEditorPerformanceModule::ShowPerformanceReportTab()
{
	FGlobalTabmanager::Get()->TryInvokeTab(FTabId(EditorDiagnosticsTabName));
}

void FEditorPerformanceModule::InitializeKPIs()
{
	// Declare the KPIs 
	{
		EditorBootKPI				= KPIRegistry.DeclareKPIValue(EditorCategoryName, EditorCategoryText, EditorBootKPIName, EditorBootKPIText, 0.0, NoSetLimit, FKPIValue::LessThan, FKPIValue::Minutes);
		EditorInitializeKPI			= //KPIRegistry.DeclareKPIValue(EditorCategoryName, EditorCategoryText, EditorInitializeKPIName, EditorInitializeKPIText, 0.0, NoSetLimit, FKPIValue::LessThan, FKPIValue::Minutes);
		EditorLoadMapKPI			= KPIRegistry.DeclareKPIValue(EditorCategoryName, EditorCategoryText, EditorLoadMapKPIName, EditorLoadMapKPIText, 0.0, NoSetLimit, FKPIValue::LessThan, FKPIValue::Minutes);
		EditorAssetRegistryScanKPI	= KPIRegistry.DeclareKPIValue(EditorCategoryName, EditorCategoryText, EditorAssetRegistryScanKPIName, EditorAssetRegistryScanKPIText, 0.0, NoSetLimit, FKPIValue::LessThan, FKPIValue::Minutes);
		EditorPluginCountKPI		= KPIRegistry.DeclareKPIValue(EditorCategoryName, EditorCategoryText, EditorPluginCountKPIName, EditorPluginCountKPIText, 0.0, NoSetLimit, FKPIValue::LessThan, FKPIValue::Decimal);
		EditorHitchRateKPI			= KPIRegistry.DeclareKPIValue(EditorCategoryName, EditorCategoryText, EditorHitchRateKPIName, EditorHitchRateKPIText, 0.0, NoSetLimit, FKPIValue::LessThan, FKPIValue::Percent);
		TotalTimeToEditorKPI		= KPIRegistry.DeclareKPIValue(EditorCategoryName, EditorCategoryText, TotalTimeToEditorKPIName, TotalTimeToEditorKPIText, 0.0, NoSetLimit, FKPIValue::LessThan, FKPIValue::Minutes);
		FKPIValue DynamicInvalidationKPIValue = FKPIValue(EditorCategoryName, EditorCategoryText, DynamicInvalidationKPIName,
			DynamicInvalidationKPIText, FKPIValue::ESeverity::Major, 0.0, DynamicInvalidationKPILimit, FKPIValue::GreaterThanOrEqual, FKPIValue::Custom);
		DynamicInvalidationKPIValue.CustomDisplayValueGetter.BindRaw(this, &FEditorPerformanceModule::GetDynamicInvalidationDisplayValue);
		DynamicInvalidationKPI		= KPIRegistry.DeclareKPIValue(MoveTemp(DynamicInvalidationKPIValue));
	}
	
	{
		PIEFirstTransitionKPI		= KPIRegistry.DeclareKPIValue(PIECategoryName, PIECategoryText, PIEFirstTransitionKPIName, PIEFirstTransitionKPIText, 0.0, NoSetLimit, FKPIValue::LessThan, FKPIValue::Minutes);
		PIETransitionKPI			= KPIRegistry.DeclareKPIValue(PIECategoryName, PIECategoryText, PIETransitionKPIName, PIETransitionKPIText, 0.0, NoSetLimit, FKPIValue::LessThan, FKPIValue::Minutes);
		PIEShutdownKPI				= KPIRegistry.DeclareKPIValue(PIECategoryName, PIECategoryText, PIEShutdownKPIName, PIEShutdownKPIText, 0.0, NoSetLimit, FKPIValue::LessThan, FKPIValue::Minutes);
		PIEHitchRateKPI				= KPIRegistry.DeclareKPIValue(PIECategoryName, PIECategoryText, PIEHitchRateKPIName, PIEHitchRateKPIText, 0.0, NoSetLimit, FKPIValue::LessThan, FKPIValue::Percent);
		TotalTimeToPIEKPI			= KPIRegistry.DeclareKPIValue(PIECategoryName, PIECategoryText, TotalTimeToPIEKPIName, TotalTimeToPIEKPIText, 0.0, NoSetLimit, FKPIValue::LessThan, FKPIValue::Minutes);
	}
	
	{
		LocalDDCEfficiencyKPI		= KPIRegistry.DeclareKPIValue(CacheCategoryName, CacheCategoryText, LocalDDCEfficiencyKPIName, LocalDDCEfficiencyKPIText, 100.0, LocalDDCEfficiencyKPILimit, FKPIValue::GreaterThan, FKPIValue::Percent);
		VirtualAssetEfficiencyKPI	= KPIRegistry.DeclareKPIValue(CacheCategoryName, CacheCategoryText, VirtualAssetEfficiencyKPIName, VirtualAssetEfficiencyKPIText, 100.0, VirtualAssetEfficiencyKPILimit, FKPIValue::GreaterThan, FKPIValue::Percent);
	}
	
	{
		AvailableMemoryKPI			= KPIRegistry.DeclareKPIValue(MemoryCategoryName, MemoryCategoryText, AvailableMemoryKPIName, AvailableMemoryKPIText, 128.0, NoSetLimit, FKPIValue::GreaterThanOrEqual, FKPIValue::GigaBytes);

		FKPIValue MemoryPressureKPIValue = FKPIValue(MemoryCategoryName, MemoryCategoryText, MemoryPressureKPIName, MemoryPressureKPIText, FKPIValue::ESeverity::Critical,
										(int)FGenericPlatformMemoryStats::EMemoryPressureStatus::Nominal, MemoryPressureKPILimit,
										FKPIValue::LessThan, FKPIValue::Custom);
		MemoryPressureKPIValue.CustomDisplayValueGetter.BindRaw(this, &FEditorPerformanceModule::GetMemoryPressureDisplayValue);

		MemoryPressureKPI			= KPIRegistry.DeclareKPIValue(MoveTemp(MemoryPressureKPIValue));
	}

	{
		FKPIValue EngineDirFreeSpaceKPIValue	= FKPIValue(DiskCategoryName, DiskCategoryText, EngineDirFreeSpaceKPIName, EngineDirFreeSpaceKPIText, FKPIValue::ESeverity::Critical,
															0.0, 5.0, FKPIValue::GreaterThanOrEqual, FKPIValue::GigaBytes);
		EngineDirFreeSpaceKPI					= KPIRegistry.DeclareKPIValue(EngineDirFreeSpaceKPIValue);

		FKPIValue ProjectDirFreeSpaceKPIValue	= FKPIValue(DiskCategoryName, DiskCategoryText, ProjectDirFreeSpaceKPIName, ProjectDirFreeSpaceKPIText, FKPIValue::ESeverity::Critical,
															0.0, 5.0, FKPIValue::GreaterThanOrEqual, FKPIValue::GigaBytes);
		ProjectDirFreeSpaceKPI					= KPIRegistry.DeclareKPIValue(ProjectDirFreeSpaceKPIValue);

		FKPIValue UserDirFreeSpaceKPIValue		= FKPIValue(DiskCategoryName, DiskCategoryText, UserDirFreeSpaceKPIName, UserDirFreeSpaceKPIText, FKPIValue::ESeverity::Critical,
															0.0, 1.0, FKPIValue::GreaterThanOrEqual, FKPIValue::GigaBytes);
		UserDirFreeSpaceKPI						= KPIRegistry.DeclareKPIValue(UserDirFreeSpaceKPIValue);

		const FText FreeSpaceHint = LOCTEXT("FreeSpaceHintMessage", "{0} has limited free space.\n\nTo avoid potential problems, please consider freeing up at least {1} GB.");
		KPIRegistry.DeclareKPIHint(EngineDirFreeSpaceKPI, FText::Format(FreeSpaceHint, FText::FromString(FPaths::ConvertRelativePathToFull(FPlatformProcess::BaseDir())), EngineDirFreeSpaceKPIValue.ThresholdValue.Get(5.0)));
		KPIRegistry.DeclareKPIHint(ProjectDirFreeSpaceKPI, FText::Format(FreeSpaceHint, FText::FromString(FPaths::ConvertRelativePathToFull(FPlatformMisc::ProjectDir())), ProjectDirFreeSpaceKPIValue.ThresholdValue.Get(5.0)));
		KPIRegistry.DeclareKPIHint(UserDirFreeSpaceKPI, FText::Format(FreeSpaceHint, FText::FromString(FPaths::ConvertRelativePathToFull(FPlatformProcess::UserDir())), UserDirFreeSpaceKPIValue.ThresholdValue.Get(1.0)));
	}

	// Declare the KPI Hints
	KPIRegistry.DeclareKPIHint(EditorBootKPI, LOCTEXT("EditorBootHintMessage", "The Editor boot time is slow.\nCheck you have enabled a Game Feature Plugin profile for your project and that the expected local cache efficiency is met.\nIf you are booting the Editor in the background then disable the Use Less CPU in Background option in the settings."));
	KPIRegistry.DeclareKPIHint(TotalTimeToEditorKPI, LOCTEXT("EditorStartupHintMessage", "The Editor start-up time is slow.\nCheck you have enabled a Game Feature Plugin profile for your project and that the expected local cache efficiency is met.\nIf you are booting the Editor in the background then disable the Use Less CPU in Background option in the settings."));
	KPIRegistry.DeclareKPIHint(EditorPluginCountKPI, LOCTEXT("EditorPluginHintMessage", "The Editor is loading more plugins than expected and this will affect Editor start-up performance.\nCheck you have enabled a Game Feature Plugin profile for your project."));

	KPIRegistry.DeclareKPIHint(PIETransitionKPI, LOCTEXT("PIETransitionHintMessage", "The Editor transition to PIE is slow.\nCheck that the expected local cache efficiency is met.\nIf you are transitioning to PIE with the Editor in the background then disable the Use Less CPU in Background option in the settings."));
	
	KPIRegistry.DeclareKPIHint(LocalDDCEfficiencyKPI, LOCTEXT("LocalCacheEfficiencyHintMessage", "The Editor will not perform well if the local cache efficiency has not yet met the expected value.\nIf this is the first time you have booted the Editor after a sync then this is to be expected."), TEXT("using-derived-data-cache-in-unreal-engine"));
	
	KPIRegistry.DeclareKPIHint(AvailableMemoryKPI, LOCTEXT("LowAvailableMemoryHintMessage", "Your hardware is running low on memory.\nTry closing applications that are no longer needed to recover available memory."));
	KPIRegistry.DeclareKPIHint(MemoryPressureKPI, LOCTEXT("MemoryPressureHintMessage", "Your system is running low on memory.\nTry closing applications that are no longer needed to recover available memory."));

	KPIRegistry.DeclareKPIHint(DynamicInvalidationKPI, LOCTEXT("DynamicInvalidationHintMessage", "Some Dynamic Invalidation Panels contain widgets that do not support invalidation."));
	
	// Load the KPI profiles
	KPIRegistry.LoadKPIProfiles(TEXT("EditorPerformance.Profile"), GEditorIni);

	// Apply any non map specific profiles
	for (FKPIProfiles::TConstIterator It(KPIRegistry.GetKPIProfiles()); It; ++It)
	{
		const FKPIProfile& Profile = It->Value;

		if (Profile.MapName.IsEmpty())
		{
			KPIProfileName = It->Key;
			KPIRegistry.ApplyKPIProfile(It->Value);
		}
	}

	EditorState = EEditorState::Editor_Boot;

	// Register the delegates
	FEditorDelegates::OnEditorBoot.AddLambda([this](double TimeToBootEditor )
		{
			EditorBootTime = (float)TimeToBootEditor;
			KPIRegistry.SetKPIValue(EditorBootKPI, EditorBootTime);
			EditorState = EEditorState::Editor_Initialize;
		});

	FEditorDelegates::OnEditorInitialized.AddLambda([this](double TimeToInitializeEditor)
		{
			EditorStartUpTime = (float)TimeToInitializeEditor-EditorLoadMapTime;
			
			KPIRegistry.SetKPIValue(EditorInitializeKPI, EditorStartUpTime-EditorBootTime );
			KPIRegistry.SetKPIValue(TotalTimeToEditorKPI, EditorStartUpTime);

			InitializeEditor();

			EditorState = EEditorState::Editor_Interact;
		});

	FEditorDelegates::OnMapLoad.AddLambda([this](const FString& MapName, FCanLoadMap& OutCanLoadMap)
		{
			LoadMapStartTime = FDateTime::UtcNow();
			IsLoadingMap = true;
		});

	FEditorDelegates::OnMapOpened.AddLambda([this](const FString& MapName, bool Unused)
		{
			IsLoadingMap = false;

			if (MapName.Len() > 0)
			{
				EditorMapName = FPaths::GetBaseFilename(MapName);

				EditorLoadMapTime = float((FDateTime::UtcNow() - LoadMapStartTime).GetTotalSeconds());
				KPIRegistry.SetKPIValue(EditorLoadMapKPI, EditorLoadMapTime);

				// Apply any profile that matches the currently loaded map
				for (FKPIProfiles::TConstIterator It(KPIRegistry.GetKPIProfiles()); It; ++It)
				{
					const FKPIProfile& Profile = It->Value;

					if (Profile.MapName == EditorMapName)
					{
						KPIProfileName = It->Key;
						KPIRegistry.ApplyKPIProfile(It->Value);
					}
				}
			}
		});

	FEditorDelegates::StartPIE.AddLambda([this](bool)
		{
			PIEStartTime = FDateTime::UtcNow();
			EditorState = EEditorState::PIE_Startup;
		});

	FWorldDelegates::OnPIEReady.AddLambda([this](UGameInstance* GameInstance)
		{	
			const float PIETransitionTime = float((FDateTime::UtcNow() - PIEStartTime).GetTotalSeconds());

			if (IsFirstTimeToPIE)
			{
				BootToPIETime = EditorStartUpTime + EditorLoadMapTime + PIETransitionTime;
				KPIRegistry.SetKPIValue(TotalTimeToPIEKPI, BootToPIETime);

				KPIRegistry.SetKPIValue(PIEFirstTransitionKPI, PIETransitionTime);
			}
			else
			{
				KPIRegistry.SetKPIValue(PIETransitionKPI, PIETransitionTime);
			}

			EditorState = EEditorState::PIE_Interact;
			IsFirstTimeToPIE = false;
		});

	FEditorDelegates::EndPIE.AddLambda([this](bool)
		{
			PIEEndTime = FDateTime::UtcNow();
			EditorState = EEditorState::PIE_Shutdown;
		});

	FEditorDelegates::ShutdownPIE.AddLambda([this](bool)
		{
			const float PIEShutdownTime = float((FDateTime::UtcNow() - PIEEndTime).GetTotalSeconds());
			KPIRegistry.SetKPIValue(PIEShutdownKPI, PIEShutdownTime);
			EditorState = EEditorState::Editor_Interact;
		});

	FModuleManager::Get().OnModulesChanged().AddLambda([this](FName ModuleName, EModuleChangeReason ChangeReason)
		{
			switch ( ChangeReason )
			{ 
				default:
				{
					break;
				}
				
				case EModuleChangeReason::ModuleLoaded:
				{
					TotalPluginCount++;

					// Hook into Asset Registry Scan callbacks as as soon as it is loaded
					if (ModuleName == TEXT("AssetRegistry"))
					{
						FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

						AssetRegistryModule.Get().OnScanStarted().AddLambda([this]()
							{
								if (EditorAssetRegistryScanCount == 0)
								{
									AssetRegistryScanStartTime = FDateTime::UtcNow();
								}	

								FPlatformAtomics::InterlockedIncrement(&EditorAssetRegistryScanCount);
							});

						AssetRegistryModule.Get().OnScanEnded().AddLambda([this]()
							{
								FPlatformAtomics::InterlockedDecrement(&EditorAssetRegistryScanCount);

								if (EditorAssetRegistryScanCount == 0)
								{
									EditorAssetRegistryScanTime = float((FDateTime::UtcNow() - AssetRegistryScanStartTime).GetTotalSeconds());
								}	
							});
					}

					break;
				}
		
				case EModuleChangeReason::ModuleUnloaded:
				{
					TotalPluginCount--;
					break;
				}	
			}
		});

#if STALL_DETECTOR

	UE::FStallDetector::StallDetected.AddLambda([this](const UE::FStallDetectedParams& Params)
		{
			FPlatformAtomics::InterlockedIncrement(&StallDetectedCount);
		});

	UE::FStallDetector::StallCompleted.AddLambda([this](const UE::FStallCompletedParams& Params)
		{
			FPlatformAtomics::InterlockedDecrement(&StallDetectedCount);
		});

#endif //STALL_DETECTOR
}

extern ENGINE_API float GAverageFPS;

void FEditorPerformanceModule::HitchSamplerCallback()
{
	static uint32 SampleCount = 0;
	static uint32 HitchCount = 0;
	static uint32 StallCount = 0;
	
	const UEditorPerformanceSettings* EditorPerformanceSettings = GetDefault<UEditorPerformanceSettings>();

	if (EditorPerformanceSettings && EditorPerformanceSettings->bEnableEditorPeformanceTool == false)
	{
		return;
	}

	// Only sample framerate and hitches and stalls when we have focus
	if (FApp::HasFocus())
	{
		if (GAverageFPS < MinFPSForHitching)
		{
			// This sample was hitching
			HitchCount++;
		}
	}

	if (StallDetectedCount > 0)
	{
		// This sample was stalling
		StallCount++;
	}

	SampleCount++;

	// Update HitchRate and StallRate after a set number of samples
	if (SampleCount > MinSamplesForHitching)
	{
		HitchRate = 100.0f * (float)HitchCount / (float)SampleCount;
		HitchCount = 0;

		StallRate = 100.0f * (float)StallCount / (float)SampleCount;
		StallCount = 0;
		
		SampleCount = 0;
	}
}

void FEditorPerformanceModule::UpdateKPIsHighFrequency()
{
	const UEditorPerformanceSettings* EditorPerformanceSettings = GetDefault<UEditorPerformanceSettings>();

	if (EditorPerformanceSettings && EditorPerformanceSettings->bEnableEditorPeformanceTool == false)
	{
		return;
	}

	// Gather memory stats
	const FPlatformMemoryStats MemoryStats = FPlatformMemory::GetStats();
	KPIRegistry.SetKPIValue(AvailableMemoryKPI, static_cast<float>(MemoryStats.AvailablePhysical) / (1024.0f * 1024.0f * 1024.0f));
	KPIRegistry.SetKPIValue(MemoryPressureKPI, static_cast<float>(MemoryStats.GetMemoryPressureStatus()));

	// Update stats that may have been captures before initialization
	KPIRegistry.SetKPIValue(EditorPluginCountKPI, (float)TotalPluginCount);

	if (EditorAssetRegistryScanCount > 0)
	{
		// Keep track of the first Asset Registry scan time
		EditorAssetRegistryScanTime = float((FDateTime::UtcNow() - AssetRegistryScanStartTime).GetTotalSeconds());
	}

	KPIRegistry.SetKPIValue(EditorAssetRegistryScanKPI, EditorAssetRegistryScanTime);

	// Gather the DDC summary stats
	FDerivedDataCacheSummaryStats SummaryStats;
	GatherDerivedDataCacheSummaryStats(SummaryStats);

	int64 TotalCloudGetHits = 0;
	int64 LocalGetHitsMisses = 0;
	float CloudLatency = 0.0;
	float CloudReadSpeed = 0.0;

	for (const FDerivedDataCacheSummaryStat& Stat : SummaryStats.Stats)
	{
		if (Stat.Key == TEXT("CloudGetHits"))
		{
			TotalCloudGetHits = FCString::Atoi(*Stat.Value);
		}
		else if (Stat.Key == TEXT("CloudLatency"))
		{
			CloudLatency = FCString::Atof(*Stat.Value);
		}
		else if (Stat.Key == TEXT("CloudReadSpeed"))
		{
			CloudReadSpeed = FCString::Atof(*Stat.Value) * 8.0f;
		}
		else if (Stat.Key == TEXT("TotalGetHitPct"))
		{
			const float Value = FCString::Atof(*Stat.Value) * 100.0f;

			if (Value > 0.0f)
			{
				KPIRegistry.SetKPIValue(TotalDDCEfficiencyKPI, Value);
			}
		}
		else if (Stat.Key == TEXT("LocalGetHitPct"))
		{
			const float Value = FCString::Atof(*Stat.Value) * 100.0f;

			if (Value > 0.0f)
			{
				KPIRegistry.SetKPIValue(LocalDDCEfficiencyKPI, Value);
			}
		}
		else if (Stat.Key == TEXT("LocalGetMisses"))
		{
			LocalGetHitsMisses = FCString::Atoi(*Stat.Value);
		}
	}

	// Gather the Virtual Assets stats
	if (UE::Virtualization::IVirtualizationSystem::Get().IsEnabled())
	{
		TArray<UE::Virtualization::FBackendStats> BackEndStatsList = UE::Virtualization::IVirtualizationSystem::Get().GetBackendStatistics();

		int64 CacheBackEndPullCount = 0;
		int64 PersistentBackEndPullCount = 0;

		for (const UE::Virtualization::FBackendStats& BackEndStats : BackEndStatsList)
		{
			switch (BackEndStats.Type)
			{
				case UE::Virtualization::EStorageType::Persistent:
				{
					PersistentBackEndPullCount += BackEndStats.PayloadActivity.Pull.PayloadCount;
					break;
				}

				default:
				case UE::Virtualization::EStorageType::Cache:
				{
					CacheBackEndPullCount += BackEndStats.PayloadActivity.Pull.PayloadCount;
					break;
				}
			}
		}

		const int64 TotalBackEndPullCount = CacheBackEndPullCount+PersistentBackEndPullCount;

		if (TotalBackEndPullCount > 0)
		{
			// Gather Virtualization analytics
			const float VirtualAssetsEfficiency = 100.0f * float(CacheBackEndPullCount) / float(TotalBackEndPullCount);
			KPIRegistry.SetKPIValue(VirtualAssetEfficiencyKPI, VirtualAssetsEfficiency);
		}
	}

	// Evaluate Cloud Cache performance
	const int64 MinimalCloudGetHits = 10;
	static int64 ElapsedCloudCacheHits = 0;
	static int64 PreviousTotalCloudGetHits = 0;
	static float AverageCloudLatency = 0;
	static float AverageCloudReadSpeed = 0;

	ElapsedCloudCacheHits = TotalCloudGetHits - PreviousTotalCloudGetHits;
	PreviousTotalCloudGetHits = TotalCloudGetHits;

	if (TotalCloudGetHits < MinimalCloudGetHits)
	{
		KPIRegistry.InvalidateKPIValue(CloudDDCLatencyKPI);
		KPIRegistry.InvalidateKPIValue(CloudDDCReadSpeedKPI);
	}
	else
	{
		KPIRegistry.SetKPIValue(CloudDDCLatencyKPI, CloudLatency);
		KPIRegistry.SetKPIValue(CloudDDCReadSpeedKPI, CloudReadSpeed);
	}

	// Record Hitch Rate
	if (EditorState == EEditorState::Editor_Interact)
	{
		KPIRegistry.SetKPIValue(EditorHitchRateKPI, HitchRate);
		KPIRegistry.SetKPIValue(EditorStallRateKPI, StallRate);
	}

	if (EditorState == EEditorState::PIE_Interact)
	{
		KPIRegistry.SetKPIValue(PIEHitchRateKPI, HitchRate);
		KPIRegistry.SetKPIValue(PIEStallRateKPI, StallRate);
	}

#if WITH_SLATE_DEBUGGING && UE_SLATE_WITH_DYNAMIC_INVALIDATION
	{
		static IConsoleVariable* DynamicInvalidationOptionsVar = IConsoleManager::Get().FindConsoleVariable(TEXT("Slate.DynamicInvalidation.Options"));
		ensure(DynamicInvalidationOptionsVar);
		const int32 DynamicInvalidationOptions = DynamicInvalidationOptionsVar ? DynamicInvalidationOptionsVar->GetInt() : 0;

		const int NoPanels = FSlateDebugging::GetNoInvalidationPanels();
		const int NoDynPanels = FSlateDebugging::GetNoDynamicInvalidationPanels();
		const int NoDynPanelsSupportingInvalidation = FSlateDebugging::GetNoDynamicInvalidationPanelsSupportingInvalidation();

		if (DynamicInvalidationOptions == 0) // Dynamic Invalidation Disabled
		{
			KPIRegistry.InvalidateKPIValue(DynamicInvalidationKPI);
		}
		else if (DynamicInvalidationOptions == 1 && NoPanels > 0) // Dynamic Invalidation Panels acting like Invalidation Panels
		{
			KPIRegistry.SetKPIValue(DynamicInvalidationKPI, 1.0); // all of them are invalidating
		}
		else if (DynamicInvalidationOptions > 1 && NoDynPanels > 0) // Dynamic Invalidation enabled or forced
		{
			KPIRegistry.SetKPIValue(DynamicInvalidationKPI, (float)NoDynPanelsSupportingInvalidation / (float)NoDynPanels);
		}
		else
		{
			KPIRegistry.InvalidateKPIValue(DynamicInvalidationKPI);
		}
	}
#else
	KPIRegistry.InvalidateKPIValue(DynamicInvalidationKPI);
#endif

	DebugForceKPIIntoBadStateIfNeeded();

	static TArray<FGuid, TInlineAllocator<4>> RecordedKPIEvents;
	
	const TArray<FGuid, TInlineAllocator<4>> RecordedKPIEventsBefore = RecordedKPIEvents;

	// Check for KPIs that have exceeded their value
	for (FKPIValues::TConstIterator It(GetKPIRegistry().GetKPIValues()); It; ++It)
	{
		const FKPIValue& KPIValue = It->Value;

		if (KPIValue.GetState() == FKPIValue::Bad)
		{
			if (RecordedKPIEvents.Find(KPIValue.Id) == INDEX_NONE)
			{
				// KPI has exceeding the threshold for the first time	
				if (EditorPerformanceSettings)
				{
					if (EditorPerformanceSettings->bEnableSnapshots)
					{
						// Create an Insights Snapshot
						RecordInsightsSnapshot(KPIValue);
					}

					if (EditorPerformanceSettings->bEnableTelemetry)
					{
						// Create a new telemetry event
						RecordTelemetryEvent(KPIValue);
					}
				}

				// Add this KPI to the list so we don't send the event again
				RecordedKPIEvents.Emplace(KPIValue.Id);
			}
		}
		else
		{
			// No longer exceeding threshold, so next time this KPI is exceeded we will record the event
			RecordedKPIEvents.Remove(KPIValue.Id);
		}
	}

	// Check TEDS diagnostic columns separately so that additions and removals both trigger a broadcast.
	bool bTedsColumnsChanged;
	{
		using namespace UE::Editor::DataStorage;
		using namespace UE::Editor::DataStorage::Queries;

		ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
		const UTedsEditorPerformanceFactory* Factory = DataStorage ? DataStorage->FindFactory<UTedsEditorPerformanceFactory>() : nullptr;

		static TSet<FGuid> PreviousTedsColumnIds;
		TSet<FGuid> CurrentTedsColumnIds;

		if (Factory)
		{
			DataStorage->RunQuery(Factory->GetDiagnosticCriticalQueryHandle(), CreateDirectQueryCallbackBinding(
				[&CurrentTedsColumnIds](IDirectQueryContext& Context, const FEditorPerformanceCriticalColumn& Column)
				{
					CurrentTedsColumnIds.Add(Column.Id);
				}));

			DataStorage->RunQuery(Factory->GetDiagnosticWarningQueryHandle(), CreateDirectQueryCallbackBinding(
				[&CurrentTedsColumnIds](IDirectQueryContext& Context, const FEditorPerformanceWarningColumn& Column)
				{
					CurrentTedsColumnIds.Add(Column.Id);
				}));
		}

		bTedsColumnsChanged = PreviousTedsColumnIds.Num() != CurrentTedsColumnIds.Num()
			|| CurrentTedsColumnIds.Difference(PreviousTedsColumnIds).Num() > 0;

		PreviousTedsColumnIds = MoveTemp(CurrentTedsColumnIds);
	}

	if (RecordedKPIEventsBefore != RecordedKPIEvents || bTedsColumnsChanged)
	{
		OnPerformanceStateChanged.Broadcast();
	}
}

void FEditorPerformanceModule::UpdateKPIsLowFrequency()
{
	const UEditorPerformanceSettings* EditorPerformanceSettings = GetDefault<UEditorPerformanceSettings>();

	if (EditorPerformanceSettings && EditorPerformanceSettings->bEnableEditorPeformanceTool == false)
	{
		return;
	}

	// Gather free disk space
	{
		uint64 TotalDiskSpace = 0;
		uint64 FreeDiskSpace = 0;

		if (FPlatformMisc::GetDiskTotalAndFreeSpace(FPlatformProcess::BaseDir(), TotalDiskSpace, FreeDiskSpace))
		{
			KPIRegistry.SetKPIValue(EngineDirFreeSpaceKPI, static_cast<float>(FreeDiskSpace) / (1024.f * 1024.f * 1024.f));
		}

		if (FPlatformMisc::GetDiskTotalAndFreeSpace(FPlatformMisc::ProjectDir(), TotalDiskSpace, FreeDiskSpace))
		{
			KPIRegistry.SetKPIValue(ProjectDirFreeSpaceKPI, static_cast<float>(FreeDiskSpace) / (1024.f * 1024.f * 1024.f));
		}

		if (FPlatformMisc::GetDiskTotalAndFreeSpace(FPlatformProcess::UserDir(), TotalDiskSpace, FreeDiskSpace))
		{
			KPIRegistry.SetKPIValue(UserDirFreeSpaceKPI, static_cast<float>(FreeDiskSpace) / (1024.f * 1024.f * 1024.f));
		}
	}

	DebugForceKPIIntoBadStateIfNeeded();
}

void FEditorPerformanceModule::DebugForceKPIIntoBadStateIfNeeded()
{
	if (!GForceBadKPIState.IsEmpty())
	{
		for (FKPIValues::TConstIterator KPIValueIt(GetKPIRegistry().GetKPIValues()); KPIValueIt; ++KPIValueIt)
		{
			const FKPIValue& KPIValue = KPIValueIt->Value;

			if (KPIValue.Path == FName(GForceBadKPIState))
			{
				const_cast<FKPIValue&>(KPIValue).State = FKPIValue::Bad;
				break;
			}
		}
	}
}

FText FEditorPerformanceModule::GetMemoryPressureDisplayValue(const float InValue) const
{
	switch (FGenericPlatformMemoryStats::EMemoryPressureStatus(InValue))
	{
		case FGenericPlatformMemoryStats::EMemoryPressureStatus::Warning:
			return LOCTEXT("MemoryPressureWarning", "Warning");
		case FGenericPlatformMemoryStats::EMemoryPressureStatus::Critical:
			return LOCTEXT("MemoryPressureCritical", "Critical");
		default:
			return LOCTEXT("MemoryPressureNormal", "Normal");
	}
}

FText FEditorPerformanceModule::GetDynamicInvalidationDisplayValue(const float InValue) const
{
#if UE_SLATE_WITH_DYNAMIC_INVALIDATION
	static IConsoleVariable* DynamicInvalidationOptionsVar = IConsoleManager::Get().FindConsoleVariable(TEXT("Slate.DynamicInvalidation.Options"));
	ensure(DynamicInvalidationOptionsVar);
	const int32 DynamicInvalidationOptions = DynamicInvalidationOptionsVar ? DynamicInvalidationOptionsVar->GetInt() : 0;
	
	if (DynamicInvalidationOptions == 0) // Dynamic Invalidation Disabled
	{
		return LOCTEXT("DynamicInvalidationDisabled", "Disabled");
	}

	const FText Percent = InValue < 0 ? FText::FromString("...") : FText::AsPercent(InValue);
	if (DynamicInvalidationOptions == 1) // Dynamic Invalidation Panels acting like regular invalidation panels
	{
		return FText::Format(LOCTEXT("DynamicInvalidationForcedStatic", "{0} (Forced Static)"), Percent);
	}
	else if (DynamicInvalidationOptions == 2) // Dynamic Invalidation enabled
	{
		return Percent;
	}
	else if (DynamicInvalidationOptions == 3) // Invalidation Panels acting like regular Dynamic Invalidation Panels
	{
		return FText::Format(LOCTEXT("DynamicInvalidationForcedDynamic", "{0} (Forced Dynamic)"), Percent);
	}

	return LOCTEXT("DynamicInvalidationUnknown", "Unknown");
#else
	return LOCTEXT("DynamicInvalidationDisabled", "Disabled");
#endif
}

bool FEditorPerformanceModule::IsHotLocalCacheCase() const
{
	FKPIValue KPIValue;

	if (KPIRegistry.GetKPIValue(LocalDDCEfficiencyKPI, KPIValue))
	{
		return KPIValue.GetState()==FKPIValue::Good;
	}

	return false;
}

bool FEditorPerformanceModule::RecordInsightsSnapshot(const FKPIValue& KPIValue)
{
	const uint32 MaxKPITraceCount = 10;

	// Create the snapshot file
	FString FileName = FString::Printf( TEXT("%s_%d.utrace"), *KPIValue.Path.ToString(), KPIValue.FailureCount % MaxKPITraceCount);
	FString FolderPath = FPaths::ProjectSavedDir() / TEXT("EditorPerformance");

	// Create the full output path
	FString FilePath = FolderPath / FileName;

	// Delete the existing trace file if it already exists
	if (FPaths::FileExists(FilePath))
	{
		if (!FPlatformFileManager::Get().GetPlatformFile().DeleteFile(*FilePath))
		{
			// File existed but could not be overwritten
			return false;
		}
	}
		
	// Write the snapshot to a file
	return FTraceAuxiliary::WriteSnapshot(*FilePath);
}

bool FEditorPerformanceModule::RecordTelemetryEvent(const FKPIValue& KPIValue)
{
	// Record a new telemetry event for this KPI
	if (FStudioTelemetry::IsAvailable())
	{
		const int SchemaVersion = 2;
		TArray<FAnalyticsEventAttribute> Attributes;

		Attributes.Emplace(TEXT("SchemaVersion"), SchemaVersion);
		Attributes.Emplace(TEXT("MapName"), EditorMapName);
		Attributes.Emplace(TEXT("DDC_IsHotLocalCache"), IsHotLocalCacheCase());
		Attributes.Emplace(TEXT("KPI_Name"), *KPIValue.Name.ToString());
		Attributes.Emplace(TEXT("KPI_Category"), *KPIValue.Category.ToString());
		Attributes.Emplace(TEXT("KPI_CurrentValue"), KPIValue.CurrentValue);
		Attributes.Emplace(TEXT("KPI_DisplayType"), *FKPIValue::GetDisplayTypeAsString(KPIValue.DisplayType));
		Attributes.Emplace(TEXT("KPI_Profile"), KPIProfileName);
		
		if (KPIValue.ThresholdValue)
		{
			Attributes.Emplace(TEXT("KPI_ThresholdValue"), KPIValue.ThresholdValue.GetValue());
		}

		FStudioTelemetry::Get().RecordEvent(TEXT("Editor.Performance.Warning"), Attributes);

		return true;
	}

	return false;
}

void FEditorPerformanceModule::TerminateKPIs()
{
}

const FKPIRegistry& FEditorPerformanceModule::GetKPIRegistry() const
{
	return KPIRegistry;
}

const FString& FEditorPerformanceModule::GetKPIProfileName() const
{
	return KPIProfileName;
}

FEditorPerformanceModule::EEditorState FEditorPerformanceModule::GetEditorState() const
{
	return EditorState;
}

#undef LOCTEXT_NAMESPACE
