// Copyright Epic Games, Inc. All Rights Reserved.

#include "KPIValue.h"
#include "Misc/ConfigCacheIni.h"

FKPIValue::EState FKPIValue::GetState() const
{
	return State;
}

void FKPIValue::UpdateState()
{
	if (!ThresholdValue)
	{
		State = EState::Good;
		return;
	}

	EState PreviousState = State;

	switch (Compare)
	{
		default:
		case FKPIValue::LessThan:
		{
			State = CurrentValue < ThresholdValue.GetValue() ? FKPIValue::Good : FKPIValue::Bad;
			break;
		}

		case FKPIValue::LessThanOrEqual:
		{
			State = CurrentValue <= ThresholdValue.GetValue() ? FKPIValue::Good : FKPIValue::Bad;
			break;
		}

		case FKPIValue::GreaterThan:
		{
			State = CurrentValue > ThresholdValue.GetValue() ? FKPIValue::Good : FKPIValue::Bad;
			break;
		}

		case FKPIValue::GreaterThanOrEqual:
		{
			State = CurrentValue >= ThresholdValue.GetValue() ? FKPIValue::Good : FKPIValue::Bad;
			break;
		}
	}
	
	if (PreviousState != State && State == FKPIValue::Bad)
	{
		FailureCount++;
	}
}

void FKPIValue::SetValue(float Value)
{
	CurrentValue = Value;

	UpdateState();
}


FString FKPIValue::GetComparisonAsString(FKPIValue::ECompare Compare )
{
	switch (Compare)
	{
		default:
		case ECompare::LessThan:
		{
			return TEXT("<");
		}

		case ECompare::LessThanOrEqual:
		{
			return TEXT("<=");
		}

		case ECompare::GreaterThan:
		{
			return TEXT(">");
		}

		case ECompare::GreaterThanOrEqual:
		{
			return TEXT(">=");
		}
	}
}

FString FKPIValue::GetComparisonAsPrettyString(FKPIValue::ECompare Compare)
{
	switch (Compare)
	{
	default:
	case ECompare::LessThan:
	{
		return TEXT("less than");
	}

	case ECompare::LessThanOrEqual:
	{
		return TEXT("less than or equal");
	}

	case ECompare::GreaterThan:
	{
		return TEXT("greater than");
	}

	case ECompare::GreaterThanOrEqual:
	{
		return TEXT("greater than or equal");
	}
	}
}

FString FKPIValue::GetDisplayTypeAsString(FKPIValue::EDisplayType DisplayType)
{
	switch (DisplayType)
	{
		default:
		case EDisplayType::Decimal:
		{
			return TEXT("Decimal");
		}

		case EDisplayType::Minutes:
		{
			return TEXT("Minutes");
		}

		case EDisplayType::Seconds:
		{
			return TEXT("Seconds");
		}

		case EDisplayType::Milliseconds:
		{
			return TEXT("Milliseconds");
		}

		case EDisplayType::Bytes:
		{
			return TEXT("Bytes");
		}

		case EDisplayType::MegaBytes:
		{
			return TEXT("MegaBytes");
		}

		case EDisplayType::GigaBytes:
		{
			return TEXT("GigaBytes");
		}

		case EDisplayType::MegaBitsPerSecond:
		{
			return TEXT("MegaBitsPerSecond");
		}

		case EDisplayType::Percent:
		{
			return TEXT("Percent");
		}
	}
}


FString	FKPIValue::GetValueAsString( float Value, FKPIValue::EDisplayType DisplayType, const FGetCustomDisplayValue& CustomDisplayValueGetter)
{
	switch (DisplayType)
	{
		default:
		{
			return FString::Printf(TEXT("%.2f"), Value);
		}

		case EDisplayType::Number:
		{
			return FString::Printf(TEXT("%d"), (int)Value);
		}

		case EDisplayType::Decimal:
		{
			return FString::Printf(TEXT("%.0f"), Value);
		}

		case EDisplayType::Minutes:
		{
			const float Minutes = FMath::Floor(Value / 60.0f);
			const float Seconds = FMath::Modulo(Value, 60.0f);
			return (Minutes > 0.0)? FString::Printf(TEXT("%.0fm %2.0fs"), Minutes, Seconds) : FString::Printf(TEXT("%2.2fs"), Seconds);
		}

		case EDisplayType::Seconds:
		{
			return FString::Printf(TEXT("%.2fs"), Value);
		}

		case EDisplayType::Milliseconds:
		{
			return FString::Printf(TEXT("%.2fms"), Value);
		}

		case EDisplayType::Bytes:
		{
			return FString::Printf(TEXT("%.2fB"), Value);
		}

		case EDisplayType::MegaBytes:
		{
			return FString::Printf(TEXT("%.2fMB"), Value);
		}

		case EDisplayType::GigaBytes:
		{
			return FString::Printf(TEXT("%.2fGB"), Value);
		}

		case EDisplayType::MegaBitsPerSecond:
		{
			return FString::Printf(TEXT("%.2fMbps"), Value);
		}
		
		case EDisplayType::Percent:
		{
			return FString::Printf(TEXT("%.2f%%"), Value);
		}

		case EDisplayType::Custom:
		{
			if (CustomDisplayValueGetter.IsBound())
			{
				return CustomDisplayValueGetter.Execute(Value).ToString();
			}
			else
			{
				return FString();
			}
		}
	}
}

FGuid FKPIRegistry::DeclareKPIValue(const FName Category, const FText DisplayCategory, const FName Name, const FText DisplayName, float InitialValue, TOptional<float> ThresholdValue, FKPIValue::ECompare Compare, FKPIValue::EDisplayType Type)
{
	return DeclareKPIValue(FKPIValue(Category, DisplayCategory, Name, DisplayName, FKPIValue::ESeverity::Minor, InitialValue, ThresholdValue, Compare, Type, FKPIValue::EState::NotSet));
}

FGuid FKPIRegistry::DeclareKPIValue( const FKPIValue& Value )
{
	if (Values.Find(Value.Id) == nullptr)
	{
		Values.Emplace(Value.Id, Value);
		return Value.Id;
	}
	return FGuid();
}

bool FKPIRegistry::DeclareKPIHint(FGuid Id, const FText& HintMessage, const FString& DocumentationLink)
{
	if (Values.Find(Id) != nullptr)
	{
		FKPIHint NewKPIHint;

		NewKPIHint.Id = Id;
		NewKPIHint.Message = HintMessage;
		NewKPIHint.DocumentationLink = DocumentationLink;

		FKPIHint* KPIHint = Hints.Find(Id);

		if (KPIHint != nullptr)
		{
			*KPIHint = NewKPIHint;
			return true;
		}
		else
		{
			Hints.Emplace(Id, NewKPIHint);
			return true;
		}
	}

	return false;
}


bool FKPIRegistry::InvalidateKPIValue(FGuid Id)
{
	FKPIValue* ExistingValue = Values.Find(Id);

	if (ExistingValue != nullptr)
	{
		ExistingValue->State = FKPIValue::NotSet;
		return true;
	}

	return false;
}

	


bool FKPIRegistry::SetKPIValue(FGuid Id, float CurrentValue)
{
	FKPIValue* ExistingValue = Values.Find(Id);

	if (ExistingValue != nullptr)
	{
		ExistingValue->SetValue(CurrentValue);
		return true;
	}	

	return false;
}

bool FKPIRegistry::SetKPIThreshold(FGuid Id, float ThresholdValue)
{
	FKPIValue* ExistingValue = Values.Find(Id);

	if (ExistingValue != nullptr)
	{
		ExistingValue->ThresholdValue = ThresholdValue;
		return true;
	}

	return false;
}

bool FKPIRegistry::GetKPIValue(FGuid Id, FKPIValue& Result) const
{
	const FKPIValue* ExistingValue = Values.Find(Id);

	if (ExistingValue != nullptr)
	{
		Result = *ExistingValue;
		return true;
	}

	return false;
}

bool FKPIRegistry::GetKPIHint(FGuid Id, FKPIHint& Result) const
{
	const FKPIHint* Hint = Hints.Find(Id);

	if (Hint != nullptr)
	{
		Result = *Hint;
		return true;
	}

	return false;
}

const TMap<FGuid, FKPIValue>& FKPIRegistry::GetKPIValues() const
{
	return Values;
}

const FKPIProfiles& FKPIRegistry::GetKPIProfiles() const
{
	return Profiles;
}

void FKPIRegistry::LoadKPIHints(const FString& HintSectionName, const FString& FileName)
{
	/*TArray<FString> SectionNames;

	if (GConfig->GetSectionNames(FileName, SectionNames))
	{
		for (const FString& SectionName : SectionNames)
		{
			if (SectionName.Find(HintSectionName) != INDEX_NONE)
			{
				FKPIHint Hint;
				Hint.URL = FText(TEXT("https://docs.unrealengine.com/5.0/en-US/"));

				for (FKPIValues::TConstIterator It(GetKPIValues()); It; ++It)
				{
					const FKPIValue& KPIValue = It->Value;
					FString KPIName = FString::Printf(TEXT("%s_%s"), *KPIValue.Category.ToString(), *KPIValue.Name.ToString()).Replace(TEXT(" "), TEXT("_"));

					if (GConfig->GetString(*SectionName, *KPIName, Hint.Message, FileName))
					{
						Hints.Emplace(It->Key, Hint);
					}
				}
			}
		}
	}*/
}

void FKPIRegistry::LoadKPIProfiles(const FString& ProfileSectionName, const FString& FileName)
{
	TArray<FString> SectionNames;

	if (GConfig->GetSectionNames(FileName, SectionNames))
	{
		for (const FString& SectionName : SectionNames)
		{
			if (SectionName.Find(ProfileSectionName) != INDEX_NONE)
			{
				FString ProfileName;
				FString MapName;

				FKPIProfile Profile;

				if (GConfig->GetString(*SectionName, TEXT("ProfileName"), ProfileName, FileName))
				{
					if (GConfig->GetString(*SectionName, TEXT("MapName"), Profile.MapName, FileName))
					{

					}

					for (FKPIValues::TConstIterator It(GetKPIValues()); It; ++It)
					{
						const FKPIValue &KPIValue = It->Value;
						
						float ThresholdValue;

						if (GConfig->GetFloat(*SectionName, *It->Value.Path.ToString(), ThresholdValue, FileName))
						{
							Profile.Thresholds.Emplace(It->Key, ThresholdValue);
						}
					}

					Profiles.Emplace(ProfileName, Profile);
				}
			}
		}
	}
}

bool FKPIRegistry::ApplyKPIProfile(const FKPIProfile& KPIProfile)
{
	bool Result = true;

	for (FKPIThesholds::TConstIterator It(KPIProfile.Thresholds); It; ++It)
	{
		Result &= SetKPIThreshold(It->Key, It->Value);
	}

	return Result;
}
