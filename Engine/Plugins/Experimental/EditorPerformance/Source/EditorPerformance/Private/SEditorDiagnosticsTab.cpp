// Copyright Epic Games, Inc. All Rights Reserved.

#include "SEditorDiagnosticsTab.h"

#include "CrashDiagnosticsModule.h"
#include "SEditorPerformanceDialogs.h"
#include "StallLogSubsystem.h"

#include "Editor/EditorEngine.h"
#include "Editor/EditorPerformanceSettings.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Styling/StyleColors.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSplitter.h"

#define LOCTEXT_NAMESPACE "EditorDiagnostics"

extern UNREALED_API UEditorEngine* GEditor;

FEditorDiagnosticsCommands::FEditorDiagnosticsCommands()
	: TCommands<FEditorDiagnosticsCommands>(
		TEXT("EditorDiagnostics"),
		NSLOCTEXT("Contexts", "EditorDiagnostics", "Editor Diagnostics"),
		NAME_None, // Parent
		FAppStyle::GetAppStyleSetName() // Icon Style Set
	)
{
}

void FEditorDiagnosticsCommands::RegisterCommands()
{	
	UI_COMMAND(SetShowActivityMonitor, "Activity", "Displays the activity monitor", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(SetShowStalls, "Stalls", "Displays the stalls log", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(SetShowCrashes, "Crashes", "Displays the crashes log", EUserInterfaceActionType::ToggleButton, FInputChord());
}

void SEditorDiagnosticsTab::Construct(const FArguments& InArgs)
{
	SDockTab::Construct(InArgs);

	TSharedRef<SBorder> ContentBorderRef =
		SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("WhiteBrush"))
		.BorderBackgroundColor(FSlateColor(EStyleColor::Panel));

	ContentBorder = ContentBorderRef;

	TSharedRef<SWidget> Splitter
		= SNew(SBox)
		.Padding(2.f)
		[
			SNew(SSplitter)
			.Orientation(Orient_Horizontal)
			.PhysicalSplitterHandleSize(2.0f)
			+SSplitter::Slot()
			.SizeRule(SSplitter::SizeToContent)
			[
				SNew(SBorder)
				.Padding(0.f)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[
					SNew(SBox)
					.MinDesiredWidth(64.f)
					[
						CreateToolBar()
					]
				]
			]
			+SSplitter::Slot()
			.SizeRule(SSplitter::FractionOfParent)
			[
				ContentBorderRef
			]
		];

	SetContent(Splitter);
}

TSharedRef<SWidget> SEditorDiagnosticsTab::CreateToolBar()
{
	TSharedRef<FUICommandList> ToolBarCommandList = MakeShared<FUICommandList>();

	bool bIsActivityMonitorAvailable = false;

	UEditorPerformanceSettings* EditorPerformanceSettings = GetMutableDefault<UEditorPerformanceSettings>();
	if (EditorPerformanceSettings && EditorPerformanceSettings->bEnableEditorPeformanceTool)
	{
		bIsActivityMonitorAvailable = true;

		TWeakPtr<FUICommandInfo> ShowActivityMonitorCommand = FEditorDiagnosticsCommands::Get().SetShowActivityMonitor;

		ToolBarCommandList->MapAction(FEditorDiagnosticsCommands::Get().SetShowActivityMonitor,
			FExecuteAction::CreateRaw(this, &SEditorDiagnosticsTab::SetShowActivityMonitor),
			FCanExecuteAction(),
			FGetActionCheckState::CreateLambda([this, ShowActivityMonitorCommand] ()
				{
					return SelectedCommand == ShowActivityMonitorCommand ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;		
				}));
	}

	TWeakPtr<FUICommandInfo> ShowStallsCommand = FEditorDiagnosticsCommands::Get().SetShowStalls;
	ToolBarCommandList->MapAction(FEditorDiagnosticsCommands::Get().SetShowStalls,
		FExecuteAction::CreateRaw(this, &SEditorDiagnosticsTab::SetShowStalls),
		FCanExecuteAction(),
		FGetActionCheckState::CreateLambda([this, ShowStallsCommand] ()
			{
				return SelectedCommand == ShowStallsCommand ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;		
			}));

	TWeakPtr<FUICommandInfo> ShowCrashesCommand = FEditorDiagnosticsCommands::Get().SetShowCrashes;
	ToolBarCommandList->MapAction(FEditorDiagnosticsCommands::Get().SetShowCrashes,
		FExecuteAction::CreateRaw(this, &SEditorDiagnosticsTab::SetShowCrashes),
		FCanExecuteAction(),
		FGetActionCheckState::CreateLambda([this, ShowCrashesCommand] ()
			{
				return SelectedCommand == ShowCrashesCommand ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			}));

	constexpr bool bForceSmallIcons = true; 
	FVerticalToolBarBuilder ToolBarBuilder = FVerticalToolBarBuilder(ToolBarCommandList, FMultiBoxCustomization::None, TSharedPtr<FExtender>(), bForceSmallIcons);
	ToolBarBuilder.SetLabelVisibility(EVisibility::Visible);

	ToolBarBuilder.SetStyle(&FAppStyle::Get(), "CategoryDrivenContentBuilderToolbarWithLabels");

	if (bIsActivityMonitorAvailable)
	{
		ToolBarBuilder.AddToolBarButton(FEditorDiagnosticsCommands::Get().SetShowActivityMonitor);
		SetShowActivityMonitor();
	}
	else
	{
		SetShowStalls();
	}

	ToolBarBuilder.AddToolBarButton(FEditorDiagnosticsCommands::Get().SetShowStalls);
	ToolBarBuilder.AddToolBarButton(FEditorDiagnosticsCommands::Get().SetShowCrashes);

	return ToolBarBuilder.MakeWidget();
}

TSharedRef<SWidget> SEditorDiagnosticsTab::CreateActivityMonitorPanel()
{
	return SNew(SEditorPerformanceReportDialog);
}

TSharedRef<SWidget> SEditorDiagnosticsTab::CreateStallLogPanel()
{
	UStallLogSubsystem* StallLogSubsystem = GEditor->GetEditorSubsystem<UStallLogSubsystem>();

	return StallLogSubsystem->CreateStallLogPanel();
}

TSharedRef<SWidget> SEditorDiagnosticsTab::CreateCrashesLogPanel()
{
	using namespace UE::Editor::CrashDiagnostics;
	FCrashDiagnosticsModule& CrashDiagnostics = FCrashDiagnosticsModule::GetChecked();
	return CrashDiagnostics.CreateCrashLogPanel();
}

void SEditorDiagnosticsTab::SetShowActivityMonitor()
{
	if (TSharedPtr<SBorder> BorderWidget = ContentBorder.Pin())
	{
		BorderWidget->SetContent(CreateActivityMonitorPanel());
		SelectedCommand = FEditorDiagnosticsCommands::Get().SetShowActivityMonitor;
	}
}

void SEditorDiagnosticsTab::SetShowStalls()
{
	if (TSharedPtr<SBorder> BorderWidget = ContentBorder.Pin())
	{
		BorderWidget->SetContent(CreateStallLogPanel());
		SelectedCommand = FEditorDiagnosticsCommands::Get().SetShowStalls;
	}
}

void SEditorDiagnosticsTab::SetShowCrashes()
{
	if (TSharedPtr<SBorder> BorderWidget = ContentBorder.Pin())
	{
		BorderWidget->SetContent(CreateCrashesLogPanel());
		SelectedCommand = FEditorDiagnosticsCommands::Get().SetShowCrashes;
	}
}

#undef LOCTEXT_NAMESPACE
