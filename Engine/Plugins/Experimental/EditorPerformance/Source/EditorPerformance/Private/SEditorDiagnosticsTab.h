// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorPerformanceModule.h"
#include "Framework/Commands/Commands.h"
#include "Widgets/Docking/SDockTab.h"

class FUICommandInfo;
class SBorder;
class SWidget;

class FEditorDiagnosticsCommands : public TCommands<FEditorDiagnosticsCommands>
{
public:
	FEditorDiagnosticsCommands();

	// TCommand<> interface
	virtual void RegisterCommands() override;
	// End of TCommand<> interface

public:
	TSharedPtr<FUICommandInfo> SetShowActivityMonitor;
	TSharedPtr<FUICommandInfo> SetShowStalls;
	TSharedPtr<FUICommandInfo> SetShowCrashes;
};

class SEditorDiagnosticsTab : public SDockTab
{
public:
	void Construct(const FArguments& InArgs);

protected:
	TSharedRef<SWidget> CreateToolBar();

	TSharedRef<SWidget> CreateActivityMonitorPanel();
	TSharedRef<SWidget> CreateStallLogPanel();
	TSharedRef<SWidget> CreateCrashesLogPanel();

	void SetShowActivityMonitor();
	void SetShowStalls();
	void SetShowCrashes();

private:
	TWeakPtr<SBorder> ContentBorder;
	TWeakPtr<FUICommandInfo> SelectedCommand;
};
