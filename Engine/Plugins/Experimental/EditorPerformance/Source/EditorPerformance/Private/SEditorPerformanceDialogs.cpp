// Copyright Epic Games, Inc. All Rights Reserved.

#include "SEditorPerformanceDialogs.h"
#include "Algo/Sort.h"
#include "Framework/Application/SlateApplication.h"
#include "Internationalization/FastDecimalFormat.h"
#include "Math/BasicMathExpressionEvaluator.h"
#include "Math/UnitConversion.h"
#include "Misc/ExpressionParser.h"
#include "Styling/StyleColors.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SSegmentedControl.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SHyperlink.h"
#include "EditorPerformanceModule.h"
#include "Modules/ModuleManager.h"
#include "IDocumentation.h"
#include "ISettingsModule.h"
#include "Editor/EditorPerformanceSettings.h"
#include "SEnumCombo.h"
#include "ToolMenus.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(SEditorPerformanceDialogs)

#define LOCTEXT_NAMESPACE "EditorPerformance"


const TCHAR* SEditorPerformanceReportDialog::SettingsMenuName = TEXT("EditorPerfSettings");

SEditorPerformanceReportDialog::~SEditorPerformanceReportDialog()
{
	if (UToolMenus* ToolMenus = UToolMenus::Get())
	{
		ToolMenus->RemoveMenu(SettingsMenuName);
	}
}

void SEditorPerformanceReportDialog::Construct(const FArguments& InArgs)
{
	const float RowMargin = 0.0f;
	const float TitleMargin = 10.0f;
	const float ColumnMargin = 10.0f;
	const FMargin DefaultMarginFirstColumn(ColumnMargin, RowMargin);
	const FSlateColor TitleColor = FStyleColors::AccentWhite;

	FEditorPerformanceModule& EditorPerfModule = FModuleManager::LoadModuleChecked<FEditorPerformanceModule>("EditorPerformance");

	this->ChildSlot
	[
		SNew(SScrollBox)
		.Orientation(Orient_Vertical)
		+ SScrollBox::Slot()
		.HAlign(HAlign_Fill)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.Padding(0, 5, 0, 0)
			.AutoHeight()
			[
				SNew(SBox)
				[
					SNew(STextBlock)
					.Margin(FMargin(TitleMargin, TitleMargin, TitleMargin, 0.f))
					.ColorAndOpacity(TitleColor)
					.Justification(ETextJustify::Left)
					.Text_Lambda([this, &EditorPerfModule]
						{ 
							return FText::Format(LOCTEXT("Profile", "Profile: {0}"), FText::FromString(*EditorPerfModule.GetKPIProfileName())); 
						}
					)
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.f)
			.Expose(KPIGridSlot)
			[
				GetKPIGridPanel()
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0, 5, 0, 0)
			.Expose(HintGridSlot)
			[
				GetHintGridPanel()
			]
		]
	];
}

EActiveTimerReturnType SEditorPerformanceReportDialog::UpdateGridPanels(double InCurrentTime, float InDeltaTime)
{
	(*KPIGridSlot)
	[
		GetKPIGridPanel()
	];

	(*HintGridSlot)
	[
		GetHintGridPanel()
	];

	SlatePrepass(GetPrepassLayoutScaleMultiplier());

	return EActiveTimerReturnType::Continue;
}

TSharedRef<SWidget> SEditorPerformanceReportDialog::GetHintGridPanel()
{
	FEditorPerformanceModule& EditorPerfModule = FModuleManager::LoadModuleChecked<FEditorPerformanceModule>("EditorPerformance");

	int32 NumHints = 0;

	TArray<FKPIHint> KPIHints;

	const FKPIValues& KPIValues = EditorPerfModule.GetKPIRegistry().GetKPIValues();

	for (FKPIValues::TConstIterator It(KPIValues); It; ++It)
	{
		const FKPIValue& KPIValue = It->Value;

		FKPIHint KPIHint;

		if (KPIValue.State == FKPIValue::Bad && EditorPerfModule.GetKPIRegistry().GetKPIHint(KPIValue.Id, KPIHint))
		{
			KPIHints.Emplace(KPIHint);
		}
	}

	TSharedRef<SGridPanel> Panel =
		SNew(SGridPanel);

	const float RowMargin = 0.0f;
	const float ColumnMargin = 10.0f;
	const FSlateColor TitleColor = FStyleColors::AccentWhite;
	const FSlateFontInfo TitleFont = FCoreStyle::GetDefaultFontStyle("Bold", 10);

	const FMargin TitleMargin(0.0f, 10.0f, ColumnMargin, 10.0f);
	const FMargin TitleMarginFirstColumn(ColumnMargin, 10.0f);
	const FMargin DefaultMargin(0.0f, RowMargin, ColumnMargin, RowMargin);
	const FMargin DefaultMarginFirstColumn(ColumnMargin, RowMargin);

	int32 Row = 0;

	if (KPIHints.Num()>0)
	{
		CurrentHintIndex = CurrentHintIndex % KPIHints.Num();

		const FKPIHint& KPIHint = KPIHints[CurrentHintIndex];

		FKPIValue KPIValue;

		if (EditorPerfModule.GetKPIRegistry().GetKPIValue(KPIHint.Id, KPIValue) == true)
		{
			Panel->AddSlot(0, Row)
				.HAlign(HAlign_Left)
				[
					SNew(STextBlock)
						.Margin(TitleMarginFirstColumn)
						.ColorAndOpacity(TitleColor)
						.Font(TitleFont)
						.Justification(ETextJustify::Left)
						.Text(LOCTEXT("HintsTitle", "Hints"))
				];

			Row++;

			Panel->AddSlot(0, Row)
				.HAlign(HAlign_Left)
				[
					SNew(STextBlock)
						.Margin(TitleMarginFirstColumn)
						.ColorAndOpacity(EStyleColor::Foreground)
						.Font(TitleFont)
						.Text(FText::Format(LOCTEXT("KPICategoryAndName", "{0} {1}"), KPIValue.DisplayCategory, KPIValue.DisplayName))
				];

			Row++;

			Panel->AddSlot(0, Row)
				.HAlign(HAlign_Left)
				[
					SNew(STextBlock)
						.AutoWrapText(true)
						.Margin(DefaultMarginFirstColumn)
						.ColorAndOpacity(EStyleColor::Foreground)
						.Justification(ETextJustify::Left)
						.Text(KPIHint.Message)
				];

			Row++;

			if (!KPIHint.DocumentationLink.IsEmpty())
			{
				Panel->AddSlot(0, Row)
					.HAlign(HAlign_Left)
					.Padding(FMargin(10.0f, 10.0f))
					[
						SNew(SHyperlink)
							.Text(LOCTEXT("HintLinkName", "Further Help & Documentation"))
							.ToolTipText(FText::FromString(*KPIHint.DocumentationLink))
							.OnNavigate_Lambda([KPIHint]() { IDocumentation::Get()->Open(KPIHint.DocumentationLink); })
					];

				Row++;
			}

			if (KPIHints.Num() > 1)
			{
				Panel->AddSlot(0, Row)
					.HAlign(HAlign_Left)
					.Padding(FMargin(10.0f, 10.0f))
					[
						SNew(SButton)
							.Text(LOCTEXT("NextHintName", "Next Hint"))
							.OnClicked_Lambda([this]()
								{
									CurrentHintIndex++;
									UpdateGridPanels(0.0f, 0.0f);
									return FReply::Handled();
								})
					];

				Row++;
			}
		}
	}

	return Panel;
}

TSharedRef<SWidget> CreateGridSlotHeaderWidget(FText SlotText, FMargin Margin, ETextJustify::Type TextJustify = ETextJustify::Left, EVisibility SeparatorVisibility = EVisibility::Visible)
{
	return SNew(SBorder)
	.BorderImage(FAppStyle::Get().GetBrush("WhiteBrush"))
	.BorderBackgroundColor(EStyleColor::Header)
	.Padding(2.f, 0.f)
	[
		SNew(SBox)
		.MinDesiredHeight(24.f)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.FillWidth(1.f)
			[
				SNew(STextBlock)
				.Margin(Margin)
				.ColorAndOpacity(EStyleColor::Foreground)
				.Text(SlotText)
				.Justification(TextJustify)
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SSeparator)
				.Thickness(1.f)
				.Orientation(Orient_Vertical)
				.Visibility(SeparatorVisibility)
			]
		]
	];
}

TSharedRef<SWidget> CreateGridSlotWidget(TAttribute<FText> SlotText, FMargin Margin, ETextJustify::Type TextJustify = ETextJustify::Left)
{
	return SNew(SBorder)
	.BorderImage(FAppStyle::Get().GetBrush("WhiteBrush"))
	.BorderBackgroundColor(EStyleColor::Background)
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("NoBorder"))
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Margin(Margin)
			.ColorAndOpacity(EStyleColor::Foreground)
			.Text(SlotText)
			.Justification(TextJustify)
		]
	];
}

TSharedRef<SWidget> SEditorPerformanceReportDialog::GetKPIGridPanel()
{
	enum EGridColumns
	{
		GridColumns_Name,
		GridColumns_CurrentValue,
		GridColumns_ExpectedValue,
		GridColumns_Failures,
		GridColumns_WarningIcon,
		GridColumns_Notify,
		GridColumns_Fill,

		GridColumns_Count
	};

	TSharedRef<SGridPanel> Panel =
		SNew(SGridPanel)
		.FillColumn(GridColumns_Fill, 1.f);

	const float RowMargin = 0.0f;
	const float ColumnMargin = 10.0f;

	const FMargin TitleMargin(ColumnMargin, 10.0f);
	const FMargin DefaultMargin(ColumnMargin, RowMargin);

	const UEditorPerformanceSettings* EditorPerformanceSettings = GetDefault<UEditorPerformanceSettings>();
	FEditorPerformanceModule& EditorPerfModule = FModuleManager::LoadModuleChecked<FEditorPerformanceModule>("EditorPerformance");

	const bool bEnableNotifications = EditorPerformanceSettings && EditorPerformanceSettings->bEnableNotifications;
	const bool bShowWarningsOnly = EditorPerformanceSettings && EditorPerformanceSettings->bShowWarningsOnly;
	
	
	int32 Row = 0;

	Panel->AddSlot(GridColumns_Name, Row)
	.HAlign(HAlign_Fill)
	.ColumnSpan(GridColumns_Count)
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(TitleMargin)
		[
			SNew(SSegmentedControl<EEditorPerformanceFilterOptions>)
			.Value(EditorPerformanceSettings->bShowWarningsOnly ? EEditorPerformanceFilterOptions::WarningsOnly : EEditorPerformanceFilterOptions::ShowAll)
			.OnValueChanged_Lambda([this](EEditorPerformanceFilterOptions OptionValue)
			{
				UEditorPerformanceSettings* EditorPerformanceSettings = GetMutableDefault<UEditorPerformanceSettings>();

				if (EditorPerformanceSettings)
				{
					EditorPerformanceSettings->bShowWarningsOnly = (OptionValue == EEditorPerformanceFilterOptions::WarningsOnly);
					EditorPerformanceSettings->PostEditChange();
					EditorPerformanceSettings->SaveConfig();
				}

				UpdateGridPanels(0.0f, 0.0f);
			})

			+ SSegmentedControl<EEditorPerformanceFilterOptions>::Slot(EEditorPerformanceFilterOptions::ShowAll)
			.Text(LOCTEXT("ShowAll", "Show All"))

			+ SSegmentedControl<EEditorPerformanceFilterOptions>::Slot(EEditorPerformanceFilterOptions::WarningsOnly)
			.Text(LOCTEXT("Warnings Only", "Warnings Only"))
		]
		+SHorizontalBox::Slot()
		.HAlign(HAlign_Right)
		.FillWidth(1.f)
		.Padding(TitleMargin)
		[
			SNew(SComboButton)
			.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButtonWithIcon")
			.HasDownArrow(false)
			.OnGetMenuContent( this, &SEditorPerformanceReportDialog::CreateSettingsMenuWidget )
			.ButtonContent()
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Icons.Settings"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		]
	];

	Row++;

	TMap<FName, TArray<FKPIValue>> SortedKPIValues;

	for (FKPIValues::TConstIterator It(EditorPerfModule.GetKPIRegistry().GetKPIValues()); It; ++It)
	{
		const FKPIValue& KPIValue = It->Value;

		if (bShowWarningsOnly && KPIValue.GetState() != FKPIValue::Bad )
		{
			continue;
		}

		if (SortedKPIValues.Find(KPIValue.Category)!=nullptr)
		{
			SortedKPIValues[KPIValue.Category].Emplace(KPIValue);
		}
		else
		{
			TArray<FKPIValue> KPIArray;
			KPIArray.Emplace(KPIValue);
			SortedKPIValues.Emplace(KPIValue.Category, KPIArray);
		}
	}

	for (TMap<FName, TArray<FKPIValue>>::TConstIterator It(SortedKPIValues); It; ++It)
	{
		const TArray<FKPIValue>& KPIValues = It->Value;

		FText CategoryText;

		if (KPIValues.Num() > 0)
		{
			CategoryText = KPIValues[0].DisplayCategory;
		}

		int32 NameColumnSpan = Row == 1 ? 1 : GridColumns_Count;
		EVisibility NameSeparatorVisibility = NameColumnSpan == 1 ? EVisibility::Visible : EVisibility::Hidden;

		// Render the category name
		Panel->AddSlot(GridColumns_Name, Row)
		.HAlign(HAlign_Fill)
		.ColumnSpan(NameColumnSpan)
		[
			CreateGridSlotHeaderWidget(CategoryText, DefaultMargin, ETextJustify::Left, NameSeparatorVisibility)
		];

		if (NameColumnSpan == 1)
		{
			Panel->AddSlot(GridColumns_CurrentValue, Row)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			[
				CreateGridSlotHeaderWidget(LOCTEXT("CurrentValueColumn", "Current"), DefaultMargin)
			];

			Panel->AddSlot(GridColumns_ExpectedValue, Row)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			[
				CreateGridSlotHeaderWidget(LOCTEXT("ExpectedValueColumn", "Expected"), DefaultMargin)
			];

			Panel->AddSlot(GridColumns_Failures, Row)
			.ColumnSpan(2)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			[
				CreateGridSlotHeaderWidget(LOCTEXT("FailedValueColumn", "Failures"), DefaultMargin)
			];

			Panel->AddSlot(GridColumns_Notify, Row)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			[
				CreateGridSlotHeaderWidget(LOCTEXT("NotifyColumn", "Notify"), DefaultMargin, ETextJustify::Center)
			];
	
			Panel->AddSlot(GridColumns_Fill, Row)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			[
				CreateGridSlotHeaderWidget(FText(), DefaultMargin, ETextJustify::Left, EVisibility::Hidden)
			];
		}

		Row++;

		for (const FKPIValue& KPIValue : KPIValues)
		{
			const FKPIValue::EState KPIValueState = KPIValue.GetState();

			const FSlateColor KPIColor = KPIValueState == FKPIValue::Bad ? EStyleColor::Warning : EStyleColor::Foreground;
			const float KPIIconSize = 8.0f;
			const FText& KPIDisplayName = KPIValue.DisplayName;
			const FName& KPIPath = KPIValue.Path;

			Panel->AddSlot(GridColumns_Name, Row)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			[
				CreateGridSlotWidget(KPIDisplayName, DefaultMargin)
			];

			Panel->AddSlot(GridColumns_CurrentValue, Row)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			[
				CreateGridSlotWidget(TAttribute<FText>::CreateLambda([KPIId = KPIValue.Id]()
					{
						FEditorPerformanceModule& EditorPerfModule = FModuleManager::LoadModuleChecked<FEditorPerformanceModule>("EditorPerformance");

						FKPIValue KPIValue;
						if (EditorPerfModule.GetKPIRegistry().GetKPIValue(KPIId, KPIValue))
						{
							if (KPIValue.State == FKPIValue::NotSet)
							{
								return LOCTEXT("PendingValue", "...");
							}
							else
							{
								return FText::FromString(*FKPIValue::GetValueAsString(KPIValue.CurrentValue, KPIValue.DisplayType, KPIValue.CustomDisplayValueGetter));
							}
						}
						else
						{
							return FText();
						}
					}), DefaultMargin)
			];

			if (KPIValue.ThresholdValue)
			{
				FText ExpectedText = FText::FromString(*(FKPIValue::GetComparisonAsString(KPIValue.Compare) + FKPIValue::GetValueAsString(KPIValue.ThresholdValue.GetValue(), KPIValue.DisplayType, KPIValue.CustomDisplayValueGetter)));

				Panel->AddSlot(GridColumns_ExpectedValue, Row)
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					CreateGridSlotWidget(ExpectedText, DefaultMargin)
				];

				Panel->AddSlot(GridColumns_Failures, Row)
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					CreateGridSlotWidget(TAttribute<FText>::CreateLambda([KPIId = KPIValue.Id]()
					{
						FEditorPerformanceModule& EditorPerfModule = FModuleManager::LoadModuleChecked<FEditorPerformanceModule>("EditorPerformance");

						FKPIValue KPIValue;
						if (EditorPerfModule.GetKPIRegistry().GetKPIValue(KPIId, KPIValue))
						{
							return FText::FromString(*FString::Printf(TEXT("%d"), KPIValue.FailureCount));
						}
						else
						{
							return FText();
						}

					}), DefaultMargin)
				];

				Panel->AddSlot(GridColumns_WarningIcon, Row)
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					SNew(SBorder)
					.BorderImage(FCoreStyle::Get().GetBrush("WhiteBrush"))
					.BorderBackgroundColor(EStyleColor::Background)
					.Padding(DefaultMargin)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
						.Image_Lambda([KPIId = KPIValue.Id]()
						{
							FEditorPerformanceModule& EditorPerfModule = FModuleManager::LoadModuleChecked<FEditorPerformanceModule>("EditorPerformance");

							FKPIValue KPIValue;
							if (EditorPerfModule.GetKPIRegistry().GetKPIValue(KPIId, KPIValue))
							{
								if (KPIValue.State == FKPIValue::Bad)
								{
									switch (KPIValue.Severity)
									{
									case FKPIValue::ESeverity::Critical:
										return FAppStyle::Get().GetBrush("EditorPerformance.Report.Critical");
									case FKPIValue::ESeverity::Major:
										return FAppStyle::Get().GetBrush("EditorPerformance.Report.Major");
									case FKPIValue::ESeverity::Minor:
									default:
										return FAppStyle::Get().GetBrush("EditorPerformance.Report.Minor");
									}
								}
								else
								{
									return FAppStyle::GetNoBrush();
								}
							}
							else
							{
								return FAppStyle::GetNoBrush();
							}
						})
					]
				];
			}
			else
			{
				Panel->AddSlot(GridColumns_ExpectedValue, Row)
				.ColumnSpan(3)
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					CreateGridSlotWidget(FText(), DefaultMargin)
				];
			}

			if (bEnableNotifications && KPIValue.ThresholdValue)
			{
				Panel->AddSlot(GridColumns_Notify, Row)
				.HAlign(HAlign_Fill)
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::Get().GetBrush("WhiteBrush"))
					.BorderBackgroundColor(EStyleColor::Background)
					[
						SNew(SBorder)
						.BorderImage(FAppStyle::Get().GetBrush("NoBorder"))
						.HAlign(HAlign_Center)
						[
							SNew(SCheckBox)
							.IsChecked_Lambda([KPIPath]() -> ECheckBoxState
								{
									UEditorPerformanceSettings* EditorPerformanceSettings = GetMutableDefault<UEditorPerformanceSettings>();
									if (EditorPerformanceSettings)
									{
										return EditorPerformanceSettings->NotifyList.Find(KPIPath) != INDEX_NONE ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
									}
									else
									{
										return ECheckBoxState::Unchecked;
									}
								})
							.OnCheckStateChanged_Lambda([this, KPIPath](ECheckBoxState NewState)
							{
								UEditorPerformanceSettings* EditorPerformanceSettings = GetMutableDefault<UEditorPerformanceSettings>();

								if (EditorPerformanceSettings)
								{
									if (NewState == ECheckBoxState::Checked)
									{
										if (EditorPerformanceSettings->NotifyList.Find(KPIPath) == INDEX_NONE)
										{
											// Add this KPI to the notification list
											EditorPerformanceSettings->NotifyList.Emplace(KPIPath);
										}
									}
									else
									{
										// Remove this KPI to the notification ignore list
										EditorPerformanceSettings->NotifyList.Remove(KPIPath);
									}

									EditorPerformanceSettings->PostEditChange();
									EditorPerformanceSettings->SaveConfig();
								}
							})
						]
					]
				];
			}
			else
			{
				Panel->AddSlot(GridColumns_Notify, Row)
				.HAlign(HAlign_Fill)
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::Get().GetBrush("WhiteBrush"))
					.BorderBackgroundColor(EStyleColor::Background)
				];
			}

			Panel->AddSlot(GridColumns_Fill, Row)
			.HAlign(HAlign_Fill)
			[
				CreateGridSlotWidget(FText(), DefaultMargin)
			];

			Row++;
		}
	}

	return Panel;
}

UToolMenu* SEditorPerformanceReportDialog::RegisterSettingsMenu()
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	if (!ToolMenus)
	{
		return nullptr;
	}

	UToolMenu* SettingsMenu = ToolMenus->RegisterMenu(SettingsMenuName);

	FToolMenuSection& SettingsMenuSection = SettingsMenu->AddSection("Settings", LOCTEXT("Settings", "Settings"));

	SettingsMenuSection.AddEntry(
		FToolMenuEntry::InitMenuEntry("Notifications",
			LOCTEXT("EnableNotificationsText", "Notifications"),
			LOCTEXT("EnableNotificationsToolTip", "Enable All Notifications"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this](){
					UEditorPerformanceSettings* EditorPerformanceSettings = GetMutableDefault<UEditorPerformanceSettings>();

					if (EditorPerformanceSettings)
					{
						EditorPerformanceSettings->bEnableNotifications = !EditorPerformanceSettings->bEnableNotifications;
						EditorPerformanceSettings->PostEditChange();
						EditorPerformanceSettings->SaveConfig();

						UpdateGridPanels(0.0f, 0.0f);
					}
				}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this]()
				{
					const UEditorPerformanceSettings* EditorPerformanceSettings = GetDefault<UEditorPerformanceSettings>();
					return EditorPerformanceSettings && EditorPerformanceSettings->bEnableNotifications;
				})
			),
			EUserInterfaceActionType::Check));

	SettingsMenuSection.AddEntry(
		FToolMenuEntry::InitMenuEntry("Snapshots",
			LOCTEXT("EnableSnapshotsText", "Snapshots"),
			LOCTEXT("EnableSnapshotsToolTip", "Enable Automatic Capture of Unreal Insights Snapshots"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this](){
					UEditorPerformanceSettings* EditorPerformanceSettings = GetMutableDefault<UEditorPerformanceSettings>();

					if (EditorPerformanceSettings)
					{
						EditorPerformanceSettings->bEnableSnapshots = !EditorPerformanceSettings->bEnableSnapshots;
						EditorPerformanceSettings->PostEditChange();
						EditorPerformanceSettings->SaveConfig();
					}

				}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this]()
				{
					const UEditorPerformanceSettings* EditorPerformanceSettings = GetDefault<UEditorPerformanceSettings>();
					return EditorPerformanceSettings && EditorPerformanceSettings->bEnableSnapshots;
				})
			),
			EUserInterfaceActionType::Check));

	SettingsMenuSection.AddEntry(
		FToolMenuEntry::InitMenuEntry("Telemetry",
			LOCTEXT("EnableTelemetryText", "Telemetry"),
			LOCTEXT("EnableTelemetryToolTip", "Record Warning Telemetry Events To Analytics System"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this](){
					UEditorPerformanceSettings* EditorPerformanceSettings = GetMutableDefault<UEditorPerformanceSettings>();

					if (EditorPerformanceSettings)
					{
						EditorPerformanceSettings->bEnableTelemetry = !EditorPerformanceSettings->bEnableTelemetry;
						EditorPerformanceSettings->PostEditChange();
						EditorPerformanceSettings->SaveConfig();
					}
				}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this]()
				{
					const UEditorPerformanceSettings* EditorPerformanceSettings = GetDefault<UEditorPerformanceSettings>();
					return EditorPerformanceSettings && EditorPerformanceSettings->bEnableTelemetry;
				})
			),
			EUserInterfaceActionType::Check));

	SettingsMenuSection.AddEntry(
		FToolMenuEntry::InitMenuEntry("Throttling",
			LOCTEXT("EnableBackgroundThrottlingText", "Throttling"),
			LOCTEXT("EnableBackgroundThrottlingToolTip", "Enable CPU throttling when the Editor is in the background."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this](){
					UEditorPerformanceSettings* EditorPerformanceSettings = GetMutableDefault<UEditorPerformanceSettings>();

					if (EditorPerformanceSettings)
					{
						EditorPerformanceSettings->bThrottleCPUWhenNotForeground = !EditorPerformanceSettings->bThrottleCPUWhenNotForeground;
						EditorPerformanceSettings->PostEditChange();
						EditorPerformanceSettings->SaveConfig();
					}
				}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this]()
				{
					const UEditorPerformanceSettings* EditorPerformanceSettings = GetDefault<UEditorPerformanceSettings>();
					return EditorPerformanceSettings && EditorPerformanceSettings->bThrottleCPUWhenNotForeground;
				})
			),
			EUserInterfaceActionType::Check));

	SettingsMenuSection.AddEntry(
		FToolMenuEntry::InitMenuEntry("Diagnostics",
			LOCTEXT("EnableShowFrameRateAndMemoryText", "Diagnostics"),
			LOCTEXT("EnableShowFrameRateAndMemoryToolTip", "Show the Frame Rate, Memory and Stalls."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this](){
					UEditorPerformanceSettings* EditorPerformanceSettings = GetMutableDefault<UEditorPerformanceSettings>();

					if (EditorPerformanceSettings)
					{
						EditorPerformanceSettings->bShowFrameRateAndMemory = !EditorPerformanceSettings->bShowFrameRateAndMemory;
						EditorPerformanceSettings->PostEditChange();
						EditorPerformanceSettings->SaveConfig();
					}
				}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this]()
				{
					const UEditorPerformanceSettings* EditorPerformanceSettings = GetDefault<UEditorPerformanceSettings>();
					return EditorPerformanceSettings && EditorPerformanceSettings->bShowFrameRateAndMemory;
				})
			),
			EUserInterfaceActionType::Check));

	SettingsMenuSection.AddSeparator(NAME_None);

	SettingsMenuSection.AddEntry(
		FToolMenuEntry::InitMenuEntry("AllSettings",
			LOCTEXT("OpenSettingsText", "More Editor Performance Settings..."),
			LOCTEXT("OpenSettingsToolTip", "Open the Editor Performance Settings Tab."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([]()
				{
					FModuleManager::LoadModuleChecked<ISettingsModule>("Settings").ShowViewer("Editor", "General", "EditorPerformanceSettings");
				}),
				FCanExecuteAction()
			)));

	return SettingsMenu;
}

TSharedRef<SWidget> SEditorPerformanceReportDialog::CreateSettingsMenuWidget()
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	if (!ToolMenus)
	{
		return SNullWidget::NullWidget;
	}

	UToolMenu* SettingsMenu = ToolMenus->FindMenu(SettingsMenuName);

	if (!SettingsMenu)
	{
		SettingsMenu = RegisterSettingsMenu();
	}

	if (SettingsMenu)
	{
		return ToolMenus->GenerateWidget(SettingsMenu);
	}
	else
	{
		return SNullWidget::NullWidget;
	}
}

#undef LOCTEXT_NAMESPACE
