// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"

#include "SEditorPerformanceDialogs.generated.h"

class SWidget;
class UToolMenu;

UENUM()
enum class EEditorPerformanceNotificationOptions : uint8
{
	Notify,
	Ignore
};

UENUM()
enum class EEditorPerformanceFilterOptions : uint8
{
	ShowAll,
	WarningsOnly
};


class SEditorPerformanceReportDialog : public SCompoundWidget
{
	static const TCHAR* SettingsMenuName;

	SLATE_BEGIN_ARGS(SEditorPerformanceReportDialog) {}
	SLATE_END_ARGS()

	virtual ~SEditorPerformanceReportDialog();

	void Construct(const FArguments& InArgs);

private:
	
	EActiveTimerReturnType UpdateGridPanels(double InCurrentTime, float InDeltaTime);
	
	TSharedRef<SWidget> GetKPIGridPanel();
	TSharedRef<SWidget> GetHintGridPanel();

	UToolMenu* RegisterSettingsMenu();
	TSharedRef<SWidget> CreateSettingsMenuWidget();

	SVerticalBox::FSlot* KPIGridSlot = nullptr;
	SVerticalBox::FSlot* HintGridSlot = nullptr;
	uint32 CurrentHintIndex = 0;
};

