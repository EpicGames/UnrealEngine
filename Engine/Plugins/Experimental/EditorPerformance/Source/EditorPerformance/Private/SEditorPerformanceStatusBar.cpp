// Copyright Epic Games, Inc. All Rights Reserved.

#include "SEditorPerformanceStatusBar.h"

#include "Async/Future.h"
#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "EditorPerformanceModule.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Framework/SlateDelegates.h"
#include "HAL/PlatformCrt.h"
#include "ISettingsModule.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Math/Color.h"
#include "Math/UnitConversion.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/Attribute.h"
#include "Misc/CoreMisc.h"
#include "Modules/ModuleManager.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateTypes.h"
#include "Textures/SlateIcon.h"
#include "ToolMenu.h"
#include "ToolMenuContext.h"
#include "ToolMenuSection.h"
#include "ToolMenus.h"
#include "Types/WidgetActiveTimerDelegate.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Settings/EditorProjectSettings.h"
#include "Editor/EditorPerformanceSettings.h"
#include "DataStorage/Features.h"
#include "Diagnostics/EditorDiagnosticsColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "TedsEditorPerformanceFactory.h"
#include "Widgets/Images/SLayeredImage.h"

class SWidget;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "EditorPerformance"


static float GKPINotificationCooldownSeconds = 300.0f;
static FAutoConsoleVariableRef CVarKPINotificationCooldownSeconds(
	TEXT("Editor.PerformanceKPI.NotificationCooldown"),
	GKPINotificationCooldownSeconds,
	TEXT("Minimum time in seconds between repeated notifications for the same KPI failure."),
	ECVF_Default);


FReply SEditorPerformanceStatusBarWidget::ViewPerformanceReport_Clicked()
{
	FModuleManager::LoadModuleChecked<FEditorPerformanceModule>("EditorPerformance").ShowPerformanceReportTab();

	return FReply::Handled();
}

void SEditorPerformanceStatusBarWidget::Construct(const FArguments& InArgs)
{
	TSharedRef<SLayeredImage> StatusIcon = SNew(SLayeredImage)
		.ColorAndOpacity(FSlateColor::UseForeground())
		.Image_Raw(this, &SEditorPerformanceStatusBarWidget::GetStatusIcon)
		.ToolTipText_Raw(this, &SEditorPerformanceStatusBarWidget::GetStatusToolTipText);
	StatusIcon->AddLayer(TAttribute<const FSlateBrush*>::CreateRaw(this, &SEditorPerformanceStatusBarWidget::GetStatusBadgeIcon));

	this->ChildSlot
		[
			SAssignNew(StatusBarButton, SButton)
			.ButtonStyle(FAppStyle::Get(), "EditorDiagnostics.StatusBarButton")
			.Content()
			[
				SNew(SBox)
				.HAlign(HAlign_Fill)
				.Padding(6.f, 0.f)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(0.f, 0.f, 3.f, 0.f)
					[
						MoveTemp(StatusIcon)
					]
					+SHorizontalBox::Slot()
					.FillWidth(1.f)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("Diagnostics", "Diagnostics"))
					]
				]
			]
			.OnClicked(FOnClicked::CreateStatic(&SEditorPerformanceStatusBarWidget::ViewPerformanceReport_Clicked))
		];

	FEditorPerformanceModule& EditorPerfModule = FModuleManager::LoadModuleChecked<FEditorPerformanceModule>("EditorPerformance");
	EditorPerfModule.GetOnPerformanceStateChanged().AddSP(this, &SEditorPerformanceStatusBarWidget::UpdateState);
}

void SEditorPerformanceStatusBarWidget::UpdateState()
{
	EditorPerformanceState = EEditorPerformanceState::Good;
	EditorPerformanceStateMessage = LOCTEXT("ToolTipMessageGood", "Editor Diagnostics: Nothing to report.");

	FEditorPerformanceModule& EditorPerfModule = FModuleManager::LoadModuleChecked<FEditorPerformanceModule>("EditorPerformance");

	const UEditorPerformanceSettings* EditorPerformanceSettings = GetDefault<UEditorPerformanceSettings>();

	WarningCount = 0;

	FText NotificationTitle;

	// Check for KPIs that have exceeded their value
	for (FKPIValues::TConstIterator It(EditorPerfModule.GetKPIRegistry().GetKPIValues()); It; ++It)
	{
		const FKPIValue& KPIValue = It->Value;

		if (KPIValue.GetState() == FKPIValue::Bad)
		{
			// Turn into a warning state if the severity warrants it
			if (KPIValue.Severity == FKPIValue::ESeverity::Major)
			{
				EditorPerformanceState = EEditorPerformanceState::Warnings;
			}
			else if (KPIValue.Severity == FKPIValue::ESeverity::Critical)
			{
				EditorPerformanceState = EEditorPerformanceState::Critical;
			}

			if (EditorPerformanceSettings && EditorPerformanceSettings->NotifyList.Find(KPIValue.Path) != INDEX_NONE)
			{
				// Turn into a warning state if the user wants to be notified
				EditorPerformanceState = FMath::Max(EEditorPerformanceState::Warnings, EditorPerformanceState);

				const double* LastShownTime = NotificationLastShownTimes.Find(KPIValue.Path);
				const bool bWithinCooldown = LastShownTime && (FPlatformTime::Seconds() - *LastShownTime < (double)GKPINotificationCooldownSeconds);

				if (AcknowledgedNotifications.Find(KPIValue.Path) == INDEX_NONE && CurrentNotificationName.IsNone() && !bWithinCooldown)
				{
					NotificationTitle = KPIValue.DisplayName;

					FKPIHint Hint;
					if (EditorPerfModule.GetKPIRegistry().GetKPIHint(KPIValue.Id, Hint))
					{
						CurrentNotificationMessage = Hint.Message;
					}
					else
					{
						CurrentNotificationMessage = FText::FromString(*FString::Printf(TEXT("%s - %s was %s but should be %s %s"),
							*KPIValue.Category.ToString(),
							*KPIValue.Name.ToString(),
							*FKPIValue::GetValueAsString(KPIValue.CurrentValue, KPIValue.DisplayType, KPIValue.CustomDisplayValueGetter),
							*FKPIValue::GetComparisonAsPrettyString(KPIValue.Compare),
							*FKPIValue::GetValueAsString(KPIValue.ThresholdValue.GetValue(), KPIValue.DisplayType, KPIValue.CustomDisplayValueGetter)));
					}

					CurrentNotificationName = KPIValue.Path;
				}
			}

			WarningCount++;
		}
		else
		{
			// No longer exceeding threshold, so no need to acknowledge the last time it was raised to the user
			// There may be subsequent times that this same KPI is exceeded this session so we may want to alert the user again
			AcknowledgedNotifications.Remove(KPIValue.Path);

			if (CurrentNotificationName == KPIValue.Path)
			{
				CurrentNotificationName = FName();
			}
		}
	}

	if (WarningCount > 0)
	{
		EditorPerformanceStateMessage = LOCTEXT("ToolTipWarning", "Editor Diagnostics: Warning. View report for details.");
	}

	// Check TEDS diagnostic columns - these override the KPI-derived state if more severe
	if (EditorPerformanceState < EEditorPerformanceState::Critical)
	{
		using namespace UE::Editor::DataStorage;
		using namespace UE::Editor::DataStorage::Queries;

		ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
		const UTedsEditorPerformanceFactory* Factory = DataStorage ? DataStorage->FindFactory<UTedsEditorPerformanceFactory>() : nullptr;
		if (Factory)
		{
			FText DiagnosticMessage;
			bool bOverrideMessage = false;

			DataStorage->RunQuery(Factory->GetDiagnosticCriticalQueryHandle(), CreateDirectQueryCallbackBinding(
				[this, &bOverrideMessage, &DiagnosticMessage](IDirectQueryContext& Context, const FEditorPerformanceCriticalColumn& ErrorColumn)
				{
					if (DiagnosticMessage.IsEmpty())
					{
						EditorPerformanceState = EEditorPerformanceState::Critical;
						DiagnosticMessage = FText::Format(LOCTEXT("EditorErrorColumnMessage", "{0}: {1}"),
							FText::FromString(FName::NameToDisplayString(ErrorColumn.Instigator.ToString(), false)), ErrorColumn.Message);
						bOverrideMessage = true;
					}
					else if (bOverrideMessage)
					{
						bOverrideMessage = false; // Keep the generic message as there are multiple issues
					}
				}));

			// Only check for warnings if we do not have critical performance issues
			if (EditorPerformanceState < EEditorPerformanceState::Warnings)
			{
				DataStorage->RunQuery(Factory->GetDiagnosticWarningQueryHandle(), CreateDirectQueryCallbackBinding(
					[this, &bOverrideMessage, &DiagnosticMessage](IDirectQueryContext& Context, const FEditorPerformanceWarningColumn& WarningColumn)
					{
						if (DiagnosticMessage.IsEmpty())
						{
							EditorPerformanceState = EEditorPerformanceState::Warnings;
							DiagnosticMessage = FText::Format(LOCTEXT("EditorWarningColumnMessage", "{0}: {1}"),
								FText::FromString(FName::NameToDisplayString(WarningColumn.Instigator.ToString(), false)), WarningColumn.Message);
							bOverrideMessage = true;
						}
						else if (bOverrideMessage)
						{
							bOverrideMessage = false; // Keep the generic message as there are multiple issues
						}
					}));
			}

			// We only override the message if we have one issue
			if (bOverrideMessage)
			{
				EditorPerformanceStateMessage = DiagnosticMessage;
			}
		}
	}

	// Only show a notification if this widget is part of a window
	const bool bIsWidgetHostedInAWindow = FSlateApplication::Get().FindWidgetWindow(SharedThis(this)).IsValid();

	if (CurrentNotificationName.IsNone() == false && EditorPerformanceSettings && EditorPerformanceSettings->bEnableNotifications && bIsWidgetHostedInAWindow)
	{	
		if (NotificationItem.IsValid() == false || NotificationItem->GetCompletionState() == SNotificationItem::CS_None)
		{
			FNotificationInfo Info(FText::Format(LOCTEXT("NotificationTitle", "{0} Warning"), NotificationTitle));

			Info.SubText = CurrentNotificationMessage;
			Info.bUseSuccessFailIcons = true;
			Info.bFireAndForget = false;
			Info.bUseThrobber = true;
			Info.FadeOutDuration = 1.0f;
			Info.ExpireDuration = 0.0f;

			// No existing notification or the existing one has finished
			TPromise<TWeakPtr<SNotificationItem>> AcknowledgeNotificationPromise;

			Info.ButtonDetails.Add(FNotificationButtonInfo(LOCTEXT("AcknowledgeNotificationButton", "Dismiss"), FText(), FSimpleDelegate::CreateLambda([NotificationFuture = AcknowledgeNotificationPromise.GetFuture().Share(),this]()
				{
					// User has acknowledged this warning
					TWeakPtr<SNotificationItem> NotificationPtr = NotificationFuture.Get();
					if (TSharedPtr<SNotificationItem> Notification = NotificationPtr.Pin())
					{
						Notification->SetCompletionState(SNotificationItem::CS_None);
						Notification->ExpireAndFadeout();
					}

					AcknowledgedNotifications.Add(CurrentNotificationName);
					CurrentNotificationName = FName();

				}), SNotificationItem::ECompletionState::CS_Fail));

			// Add a "Don't show this again" option
			Info.CheckBoxState = TAttribute<ECheckBoxState>::CreateLambda([CurrentNotificationName=this->CurrentNotificationName]()
				{
					if (CurrentNotificationName.IsNone() == false)
					{
						if (const UEditorPerformanceSettings* EditorPerformanceSettings = GetDefault<UEditorPerformanceSettings>())
						{
							return EditorPerformanceSettings->NotifyList.Contains(CurrentNotificationName) ? ECheckBoxState::Unchecked : ECheckBoxState::Checked;
						}
					}

					return ECheckBoxState::Unchecked;
				});
			Info.CheckBoxStateChanged = FOnCheckStateChanged::CreateLambda([CurrentNotificationName=this->CurrentNotificationName](ECheckBoxState NewState)
				{
					if (CurrentNotificationName.IsNone())
					{
						return;
					}

					if (UEditorPerformanceSettings* EditorPerformanceSettings = GetMutableDefault<UEditorPerformanceSettings>())
					{
						switch (NewState)
						{
							case ECheckBoxState::Checked:
								EditorPerformanceSettings->NotifyList.Remove(CurrentNotificationName);
								break;
							case ECheckBoxState::Unchecked:
								EditorPerformanceSettings->NotifyList.AddUnique(CurrentNotificationName);
								break;
							default:
								break;
						}

						EditorPerformanceSettings->PostEditChange();
						EditorPerformanceSettings->SaveConfig();
					}
				});
			Info.CheckBoxText = LOCTEXT("DontShowThisAgainCheckBoxMessage", "Don't show this again");


			// Create the notification item
			NotificationItem = FSlateNotificationManager::Get().AddNotification(Info);

			if (NotificationItem.IsValid())
			{
				AcknowledgeNotificationPromise.SetValue(NotificationItem);
				NotificationItem->SetCompletionState(SNotificationItem::CS_Fail);
				NotificationLastShownTimes.Add(CurrentNotificationName, FPlatformTime::Seconds());
			}
		}
	}
	// No longer any warnings so kill any existing notifications
	else if (NotificationItem.IsValid())
	{
		NotificationItem->SetCompletionState(SNotificationItem::CS_None);
		NotificationItem->ExpireAndFadeout();
	}

	if (StatusBarButton)
	{
		StatusBarButton->SetButtonStyle(GetButtonStyle());
	}
}

const FSlateBrush* SEditorPerformanceStatusBarWidget::GetStatusIcon() const
{
	switch (EditorPerformanceState)
	{
		default:
		case EEditorPerformanceState::Good:
		{
			return FAppStyle::Get().GetBrush("EditorDiagnostics.StatusBar.Icon");
		}

		case EEditorPerformanceState::Warnings:
		{
			return FAppStyle::Get().GetBrush("EditorDiagnostics.StatusBar.BadgeBG");
		}
		case EEditorPerformanceState::Critical:
		{
			return FAppStyle::Get().GetBrush("Icons.Alert.Solid");
		}
	}
}

const FSlateBrush* SEditorPerformanceStatusBarWidget::GetStatusBadgeIcon() const
{
	switch (EditorPerformanceState)
	{
		default:
		case EEditorPerformanceState::Good:
		case EEditorPerformanceState::Critical:
		{
			return FAppStyle::Get().GetBrush("NoBrush");
		}

		case EEditorPerformanceState::Warnings:
		{
			return FAppStyle::Get().GetBrush("EditorDiagnostics.StatusBar.WarningBadge");
		}
	}
}

FText SEditorPerformanceStatusBarWidget::GetStatusToolTipText() const
{
	return EditorPerformanceStateMessage;
}

const FButtonStyle* SEditorPerformanceStatusBarWidget::GetButtonStyle() const
{
	switch (EditorPerformanceState)
	{
		default:
		case EEditorPerformanceState::Good:
		case EEditorPerformanceState::Warnings:
		{
			return &FAppStyle::Get().GetWidgetStyle<FButtonStyle>("EditorDiagnostics.StatusBarButton");
		}

		case EEditorPerformanceState::Critical:
		{
			return &FAppStyle::Get().GetWidgetStyle<FButtonStyle>("EditorDiagnostics.StatusBarButtonAlert");
		}
	}
}

#undef LOCTEXT_NAMESPACE
