// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

struct FButtonStyle;
class FUICommandInfo;
class FUICommandList;
class SButton;
class SNotificationItem;
class SWidget;
class FKPIValue;
struct FSlateBrush;


enum class EEditorPerformanceState : uint8
{
	Good,
	Warnings,
	Critical
};

class SEditorPerformanceStatusBarWidget : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SEditorPerformanceStatusBarWidget) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:
	static FReply ViewPerformanceReport_Clicked();

	const FSlateBrush*				GetStatusIcon() const;
	const FSlateBrush*				GetStatusBadgeIcon() const;
	FText							GetStatusToolTipText() const;
	const FButtonStyle*				GetButtonStyle() const;

	void							UpdateState();
	
	TSharedPtr<SNotificationItem>	NotificationItem;
	EEditorPerformanceState			EditorPerformanceState= EEditorPerformanceState::Good;
	FText							EditorPerformanceStateMessage;
	FText							CurrentNotificationMessage;
	FName							CurrentNotificationName;
	TArray<FName>					AcknowledgedNotifications;
	TMap<FName, double>				NotificationLastShownTimes;
	uint32							WarningCount = 0;

	TSharedPtr<SButton>				StatusBarButton;
};

