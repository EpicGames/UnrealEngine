// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsEditorPerformanceFactory.h"

#include "Diagnostics/EditorDiagnosticsColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"

void UTedsEditorPerformanceFactory::RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::Queries;

	DiagnosticWarningQueryHandle = DataStorage.RegisterQuery(
		Select()
			.ReadOnly<FEditorPerformanceWarningColumn>()
		.Where()
		.Compile());

	DiagnosticCriticalQueryHandle = DataStorage.RegisterQuery(
		Select()
			.ReadOnly<FEditorPerformanceCriticalColumn>()
		.Where()
		.Compile());
}