// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/Handles.h"
#include "Elements/Interfaces/TypedElementDataStorageFactory.h"

#include "TedsEditorPerformanceFactory.generated.h"

UCLASS()
class UTedsEditorPerformanceFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	virtual void RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;

	UE::Editor::DataStorage::QueryHandle GetDiagnosticWarningQueryHandle() const { return DiagnosticWarningQueryHandle; }
	UE::Editor::DataStorage::QueryHandle GetDiagnosticCriticalQueryHandle() const { return DiagnosticCriticalQueryHandle; }

private:
	UE::Editor::DataStorage::QueryHandle DiagnosticWarningQueryHandle = UE::Editor::DataStorage::InvalidQueryHandle;
	UE::Editor::DataStorage::QueryHandle DiagnosticCriticalQueryHandle = UE::Editor::DataStorage::InvalidQueryHandle;
};