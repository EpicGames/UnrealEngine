// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Engine/EngineTypes.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"
#include "KPIValue.h"

#define UE_API EDITORPERFORMANCE_API

class FSpawnTabArgs;
class SDockTab;
class SWidget;
class SWindow;
struct FTimerHandle;

/**
 * The module holding all of the UI related pieces for EditorPerformance
 */
class FEditorPerformanceModule : public IModuleInterface
{
public:

	enum EEditorState : uint8
	{
		Editor_Boot,
		Editor_Initialize,
		Editor_Interact,
		PIE_Startup,
		PIE_Interact,
		PIE_Shutdown,
	};

	/**
	 * Called right after the module DLL has been loaded and the module object has been created
	 */
	UE_API virtual void StartupModule() override;

	/**
	 * Called before the module is unloaded, right before the module object is destroyed.
	 */
	UE_API virtual void ShutdownModule() override;

	UE_API TSharedRef<SWidget> CreateStatusBarWidget();
	UE_API void ShowPerformanceReportTab();

	UE_API const FKPIRegistry& GetKPIRegistry() const;
	UE_API const FString& GetKPIProfileName() const;

	UE_API bool RecordInsightsSnapshot(const FKPIValue& Value);
	UE_API bool RecordTelemetryEvent(const FKPIValue& Value);
	UE_API bool IsHotLocalCacheCase() const;
	UE_API FEditorPerformanceModule::EEditorState GetEditorState() const;

	DECLARE_EVENT(FEditorPerformanceModule, FPerformanceStateChangedEvent);
	FPerformanceStateChangedEvent& GetOnPerformanceStateChanged() { return OnPerformanceStateChanged; }

private:
	TSharedRef<SDockTab> CreatePerformanceReportTab(const FSpawnTabArgs& Args);	
	TWeakPtr<SDockTab> PerformanceReportTab;

	void InitializeEditor();
	void TerminateEditor();

	void InitializeKPIs();
	void TerminateKPIs();

	void UpdateKPIsHighFrequency();
	void UpdateKPIsLowFrequency();
	void HitchSamplerCallback();

	void DebugForceKPIIntoBadStateIfNeeded();

	FText GetMemoryPressureDisplayValue(const float) const;
	FText GetDynamicInvalidationDisplayValue(const float) const;

	FKPIRegistry					KPIRegistry;
	TMap<FString,FKPIProfile>		KPIProfiles;
	FString							KPIProfileName=TEXT("Default");

	FPerformanceStateChangedEvent	OnPerformanceStateChanged;

	FDateTime						LoadMapStartTime;
	FDateTime						PIEStartTime;
	FDateTime						PIEEndTime;
	FDateTime						AssetRegistryScanStartTime; 
	bool							IsFirstTimeToPIE = true;
	bool							IsLoadingMap = false;
	EEditorState					EditorState = EEditorState::Editor_Boot;
	float							BootToPIETime=0;
	float							EditorBootTime = 0;
	float							EditorStartUpTime = 0;
	float							EditorLoadMapTime = 0;
	float							EditorAssetRegistryScanTime = 0;
	volatile int32					EditorAssetRegistryScanCount = 0;
	FString							EditorMapName=TEXT("Boot");

	FTimerHandle					HighFrequencyUpdateTimerHandle;
	const float						HighFrequencyUpdateIntervalSeconds = 1.0f;
	FTimerHandle					LowFrequencyUpdateTimerHandle;
	const float						LowFrequencyUpdateIntervalSeconds = 60.0f;

	FTimerHandle					HitchSamplerTimerHandle;
	const float						HitchSamplerIntervalSeconds = 0.1f;
	const float						MinFPSForHitching = 15.0f;
	const uint32					MinSamplesForHitching = 10;
	volatile int32					StallDetectedCount = 0;
	float							HitchRate = 0;
	float							StallRate = 0;
	uint32							TotalPluginCount = 0;

	FGuid							EditorBootKPI;
	FGuid							EditorInitializeKPI;
	FGuid							EditorLoadMapKPI;
	FGuid							EditorHitchRateKPI;
	FGuid							EditorStallRateKPI;
	FGuid							EditorAssetRegistryScanKPI;
	FGuid							EditorPluginCountKPI;
	FGuid							TotalTimeToEditorKPI;
	FGuid							TotalTimeToPIEKPI;
	FGuid							PIEFirstTransitionKPI;
	FGuid							PIETransitionKPI;
	FGuid							PIEShutdownKPI;
	FGuid							PIEHitchRateKPI;
	FGuid							PIEStallRateKPI;
	FGuid							CloudDDCLatencyKPI;
	FGuid							CloudDDCReadSpeedKPI;
	FGuid							TotalDDCEfficiencyKPI;
	FGuid							LocalDDCEfficiencyKPI;
	FGuid							VirtualAssetEfficiencyKPI;
	FGuid							AvailableMemoryKPI;
	FGuid							MemoryPressureKPI;
	FGuid							DynamicInvalidationKPI;

	FGuid							EngineDirFreeSpaceKPI;
	FGuid							ProjectDirFreeSpaceKPI;
	FGuid							UserDirFreeSpaceKPI;
};


#undef UE_API
