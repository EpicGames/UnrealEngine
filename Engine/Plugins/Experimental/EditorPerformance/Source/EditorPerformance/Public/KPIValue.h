// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/UnrealString.h"
#include "Containers/Array.h"
#include "Containers/Map.h"

class FKPIValue
{
public:
	DECLARE_DELEGATE_RetVal_OneParam(FText, FGetCustomDisplayValue, const float);

	enum ECompare : uint8
	{
		LessThan,
		LessThanOrEqual,
		GreaterThan,
		GreaterThanOrEqual,
	};

	enum EDisplayType : uint8
	{
		Number,
		Decimal,
		Seconds,
		Milliseconds,
		Minutes,
		Bytes,
		MegaBytes,
		GigaBytes,
		MegaBitsPerSecond,
		Percent,
		Custom
	};

	enum EState : uint8
	{
		NotSet,
		Good,
		Bad,
	};

	enum ESeverity : uint8
	{
		Minor,
		Major,
		Critical,
	};

	FKPIValue(FName NewCategory, FText NewDisplayCategory, FName NewName, FText NewDisplayName, FKPIValue::ESeverity NewSeverity, float NewInitialValue, TOptional<float> NewThresholdValue,
		FKPIValue::ECompare NewCompare = ECompare::LessThan, FKPIValue::EDisplayType NewDisplayType=EDisplayType::Number, FKPIValue::EState NewState=NotSet):
		Id(FGuid::NewGuid()),
		Category(NewCategory),
		DisplayCategory(NewDisplayCategory),
		Name(NewName),
		DisplayName(NewDisplayName),
		Severity(NewSeverity),
		CurrentValue(NewInitialValue),
		ThresholdValue(NewThresholdValue),
		State(NewState),
		Compare(NewCompare),
		DisplayType(NewDisplayType)	
	{
		Path = FName(FString::Printf(TEXT("%s_%s"), *Category.ToString(), *Name.ToString()).Replace(TEXT(" "), TEXT("_")));
	}

	FKPIValue()
	{}

	EState				GetState() const;
	void				SetValue(float Value);
	static FString		GetValueAsString(float Value, FKPIValue::EDisplayType Type, const FGetCustomDisplayValue& CustomDisplayValueGetter);
	static FString		GetComparisonAsString(FKPIValue::ECompare Compare);
	static FString		GetComparisonAsPrettyString(FKPIValue::ECompare Compare);
	static FString		GetDisplayTypeAsString(FKPIValue::EDisplayType Type);

	FGuid				Id;

	FName				Category;
	FText				DisplayCategory;

	FName				Name;
	FText				DisplayName;

	FName				Path;

	ESeverity			Severity;

	float				CurrentValue = 0;
	TOptional<float>	ThresholdValue;
	uint32				FailureCount = 0;
	EState				State = EState::NotSet;
	ECompare			Compare = ECompare::LessThan;
	EDisplayType		DisplayType = EDisplayType::Number;
	FGetCustomDisplayValue	CustomDisplayValueGetter;

protected:
	void UpdateState();

};

typedef TMap<FGuid, FKPIValue> FKPIValues;
typedef TMap<FGuid, float> FKPIThesholds;

class FKPIProfile
{
public:
	FString				MapName=TEXT("");
	FKPIThesholds		Thresholds;
};

typedef TMap<FString, FKPIProfile> FKPIProfiles;

class FKPIHint
{
public:
	FGuid			Id;
	FText			Message;
	FString			DocumentationLink;
};

typedef TMap<FGuid, FKPIHint> FKPIHints;

class FKPIRegistry
{
public:

	FGuid							DeclareKPIValue(const FName Category, const FText DisplayCategory, const FName Name, const FText DisplayName, float InitialValue, TOptional<float> ThresholdValue, FKPIValue::ECompare Compare, FKPIValue::EDisplayType Type);
	FGuid							DeclareKPIValue(const FKPIValue& Value);
	bool							DeclareKPIHint(FGuid Id, const FText& HintMessage, const FString& DocumentationLink = FString());

	bool							SetKPIValue(FGuid Id, float CurrentValue);
	bool							SetKPIThreshold(FGuid Id, float ThresholdValue);
	bool							InvalidateKPIValue(FGuid Id);
	bool 							GetKPIValue(FGuid Id, FKPIValue& Result) const;
	bool 							GetKPIHint(FGuid Id, FKPIHint& Result) const;
	const FKPIValues&				GetKPIValues() const;
	const FKPIProfiles&				GetKPIProfiles() const;
	
	void							LoadKPIHints(const FString& HintSectionName, const FString& FileName);
	void							LoadKPIProfiles(const FString& ProfileSectionName, const FString& FileName);
	bool							ApplyKPIProfile(const FKPIProfile& Profile);

private:

	FKPIValues						Values;
	FKPIProfiles					Profiles;
	FKPIHints						Hints;
};

