// Copyright Epic Games, Inc. All Rights Reserved.

#include "StallLogSubsystem.h"

#include "Async/TaskGraphInterfaces.h"
#include "Containers/Array.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Notifications/NotificationManager.h"
#include "GenericPlatform/GenericPlatformCrashContext.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformApplicationMisc.h"
#include "HAL/ThreadManager.h"
#include "Logging/MessageLog.h"
#include "MessageLogInitializationOptions.h"
#include "MessageLogModule.h"
#include "Modules/BuildVersion.h"
#include "Modules/ModuleManager.h"
#include "ProfilingDebugging/StallDetector.h"
#include "SlateOptMacros.h"
#include "Stats/Stats.h"
#include "ToolMenus.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SHeader.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StallLogSubsystem)

#define LOCTEXT_NAMESPACE "StallLogSubsystem"

class FStallLogSubsystemModule : public IModuleInterface {

};

IMPLEMENT_MODULE(FStallLogSubsystemModule, StallLogSubsystem);

namespace 
{
	bool EnableStallLogSubsystem = true;
	bool EnableStallLogStackTracing = true;
}

static FAutoConsoleVariableRef CVarEnableStallLogging(
	TEXT("Editor.StallLogger.Enable"),
	EnableStallLogSubsystem,
	TEXT("Whether the editor stall logger subsystem is enabled."),
	ECVF_Default);


static FAutoConsoleVariableRef CVarEnableStallStackTracing(
	TEXT("Editor.StallLogger.EnableStackTrace"),
	EnableStallLogStackTracing,
	TEXT("When enabled, stall logs will do a stacktrace to get human-readable function history"),
	ECVF_Default);

/**
 * Metadata for each detected stall of the application
 */
class FStallLogItem : public TSharedFromThis<FStallLogItem>
{
public:
	FStallLogItem() = default;

	// Constructor for OnStallDetected firing first
	FStallLogItem(FDateTime InDetectTime, TConstArrayView<uint64> InBackTrace)
		: Location()
		, ThreadName()
		, DurationSeconds()
		, Time(InDetectTime)
		, BackTrace(InBackTrace)
	{
	}

	// Constructor for OnStallDetected firing second
	FStallLogItem(FStallLogItem&& Old, FDateTime InDetectTime, TConstArrayView<uint64> InBackTrace)
		: Location(MoveTemp(Old.Location))
		, ThreadName(MoveTemp(Old.ThreadName))
		, DurationSeconds(Old.DurationSeconds)
		, Time(InDetectTime)
		, BackTrace(InBackTrace)
	{
	}

	// Constructor for OnStallComplete firing first
	FStallLogItem(FStringView Location, FString InThreadName, float DurationSeconds)
		: Location(Location)
		, ThreadName(InThreadName)
		, DurationSeconds(DurationSeconds)
		, Time()
		, BackTrace()
	{
	}

	// Constructor for OnStallComplete firing second
	FStallLogItem(FStallLogItem&& Old, FStringView Location, FString InThreadName, float DurationSeconds)
		: Location(Location)
		, ThreadName(InThreadName)
		, DurationSeconds(DurationSeconds)
		, Time(Old.Time)
		, BackTrace(MoveTemp(Old.BackTrace))
	{
	}

	// Constructor for OnstallComplte firing without background detection
	FStallLogItem(FStringView InLocation, FString ThreadName, float InDurationSeconds, FDateTime InTime)
		: Location(InLocation)
		, ThreadName(MoveTemp(ThreadName))
		, DurationSeconds(InDurationSeconds)
		, Time(InTime)
		, BackTrace()
	{
	}

	FString Location;
	FString ThreadName;
	float DurationSeconds;
	FDateTime Time;
	TArray<uint64> BackTrace;
};

using FStallLogItemPtr = TSharedPtr<FStallLogItem>;

/**
 * Holds a history of all the detected stalls
 * Model used for the UI
 */
class FStallLogHistory
{
public:
	void OnStallDetected(uint64 UniqueID, FDateTime InDetectTime, TConstArrayView<uint64> Backtrace);
	void OnStallCompleted(uint32 ThreadID, FStringView StatName, uint64 UniqueID, FDateTime InCompletedTime, float InDurationSeconds, bool bWasDetectedWithCallstack);

	void SetHasViewedStalls(bool bHasViewed) { bHasViewedStalls = bHasViewed; };
	bool GetHasViewedStalls() { return bHasViewedStalls; };

	void AddStallToLog(FStallLogItemPtr LogItemToAdd);
	void ClearStallLog();
	
	const TArray<FStallLogItemPtr>& GetStallLog() const;

private:
	// Stalls for which we have received only one of the necessary callbacks, not yet added to StallLogs
	TMap<uint64, FStallLogItem> InFlightStalls;
	// Completed stalls with full information
	TArray<FStallLogItemPtr> StallLogs;
	// If the user has viewed all current stall logs, set to false when a new stall is added
	bool bHasViewedStalls = true;
};

const TArray<FStallLogItemPtr>& FStallLogHistory::GetStallLog() const
{
	checkf(IsInGameThread(), TEXT("Can only be run on GameThread"));
	return StallLogs;
}

void FStallLogHistory::ClearStallLog()
{
	checkf(IsInGameThread(), TEXT("Can only be run on GameThread"));
	StallLogs.Empty();
	SetHasViewedStalls(true);
}

namespace
{
	const FName ColumnName_Location = FName(TEXT("Location"));
	const FName ColumnName_Thread = FName(TEXT("Thread"));
	const FName ColumnName_Duration = FName(TEXT("Duration"));
	const FName ColumnName_Time = FName(TEXT("Time"));
	const FName ColumnName_Copy = FName(TEXT("Copy"));

	DECLARE_DELEGATE(FStallLogClearLog)
	DECLARE_DELEGATE_RetVal(const FSlateBrush*, FGetSlateBrush);
	
	/**
	 * A widget for each row of the stall stable
	 */
	class SStallLogItemRow
		: public SMultiColumnTableRow<FStallLogItemPtr>
	{
	public:
		SLATE_BEGIN_ARGS(SStallLogItemRow) {}
		SLATE_END_ARGS()

	public:

		void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, const FStallLogItemPtr& InStallLogItem)
		{
			StallLogItem = InStallLogItem;
			SMultiColumnTableRow<FStallLogItemPtr>::Construct(FSuperRowType::FArguments(), InOwnerTableView);
		}
		
		virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
		{
			if (ColumnName == ColumnName_Location)
			{
				return SNew(SBox)
					.Padding(FMargin(4.0f, 0.0f))
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
							.ColorAndOpacity(FSlateColor::UseForeground())
							.Text(FText::FromString(StallLogItem->Location))
					];
			}
			else if (ColumnName == ColumnName_Thread)
			{
				return SNew(SBox)
					.Padding(FMargin(4.0f, 0.0f))
					.VAlign(VAlign_Center)
						[SNew(STextBlock)
								.ColorAndOpacity(FSlateColor::UseForeground())
								.Text(FText::FromString(StallLogItem->ThreadName))];
			}
			else if (ColumnName == ColumnName_Duration)
			{
				return SNew(SBox)
					.Padding(FMargin(4.0f, 0.0f))
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
							.ColorAndOpacity(FSlateColor::UseForeground())
							.Text(FText::Format(LOCTEXT("DurationFmt", "{0}"), StallLogItem->DurationSeconds))
					];
			}
			else if (ColumnName == ColumnName_Time)
			{
				return SNew(SBox)
					.Padding(FMargin(4.0f, 0.0f))
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
							.ColorAndOpacity(FSlateColor::UseForeground())
							.Text(FText::Format(LOCTEXT("TimeFmt", "{0}"), FText::AsDateTime(StallLogItem->Time, EDateTimeStyle::Default, EDateTimeStyle::Default, FText::GetInvariantTimeZone())))
					];
			}
			else if (ColumnName == ColumnName_Copy)
			{
				return
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				[
					SNew(SSpacer)
				]
				+SHorizontalBox::Slot()
				.MaxWidth(16)
				[
					SNew(SButton)
					.ToolTipText(LOCTEXT("StallDetector", "Copy Stall Information"))
					.ButtonStyle(FAppStyle::Get(), "NoBorder")
					.ContentPadding(FMargin(0.0f))
					.Visibility(EVisibility::Visible)
					.OnClicked_Lambda([this]()
					{
						// Build a string of the stall information and put it on the clipboard
						TStringBuilder<32768> Clipboard;
						
						FBuildVersion BuildVersion;
						if (FBuildVersion::TryRead(FBuildVersion::GetDefaultFileName(), BuildVersion))
						{
							Clipboard.Appendf(TEXT("Engine Version: %s\n"), *BuildVersion.GetEngineVersion().ToString());
						}
						else
						{
							Clipboard.Append(TEXT("Engine Version: <unknown>\n"));
						}
						Clipboard.Appendf(TEXT("Stall Detector: %s\n"), *StallLogItem->Location);
						Clipboard.Appendf(TEXT("Stall Duration: %03f\n"), StallLogItem->DurationSeconds);
						Clipboard.Appendf(TEXT("Stall Time: %s\n"), *StallLogItem->Time.ToString());
						
						if (!EnableStallLogStackTracing)
						{
							Clipboard.Append(TEXT("BackTrace\n=========\n"));
							for (int32 StackFrameIndex = 0; StackFrameIndex < StallLogItem->BackTrace.Num(); ++StackFrameIndex)
							{
								const uint64 ProgramCounter = StallLogItem->BackTrace[StackFrameIndex];
								Clipboard.Appendf(TEXT("%02d: [0x%p]\n"), StackFrameIndex, ProgramCounter);
							}
							Clipboard.Append(TEXT("\nUse CMD: \"Editor.StallLogger.EnableStackTrace = 1\" to enable more detailed stacktrace\"\n"));
						}
						else // !GEnableStallLogStackTracing
						{
							Clipboard.Append(TEXT("\nStackTrace\n==========\n"));
							const bool StackWalkingInitialized = FPlatformStackWalk::InitStackWalking();

							if (StackWalkingInitialized)
							{
								for (int32 StackFrameIndex = 0; StackFrameIndex < StallLogItem->BackTrace.Num(); ++StackFrameIndex)
								{
									FProgramCounterSymbolInfo SymbolInfo;
									const uint64 ProgramCounter = StallLogItem->BackTrace[StackFrameIndex];
									FPlatformStackWalk::ProgramCounterToSymbolInfo(ProgramCounter, SymbolInfo);

									// Strip module path.
									const ANSICHAR* Pos0 = FCStringAnsi::Strrchr( SymbolInfo.ModuleName, '\\' );
									const ANSICHAR* Pos1 = FCStringAnsi::Strrchr( SymbolInfo.ModuleName, '/' );
									const UPTRINT RealPos = FMath::Max(reinterpret_cast<UPTRINT>(Pos0), reinterpret_cast<UPTRINT>(Pos1) );
									const ANSICHAR* StrippedModuleName = RealPos > 0 ? reinterpret_cast<const ANSICHAR*>(RealPos + 1) : SymbolInfo.ModuleName;
											
									Clipboard.Appendf(TEXT("%02d: [0x%p] [%s] : [%s] : <%s>:%d\n"),
										StackFrameIndex,
										ProgramCounter,
										ANSI_TO_TCHAR(StrippedModuleName),
										ANSI_TO_TCHAR(SymbolInfo.FunctionName),
										ANSI_TO_TCHAR(SymbolInfo.Filename),
										SymbolInfo.LineNumber);
								}
							}
							else
							{
								Clipboard.Append(TEXT("- Failed to initialize StackWalking\n"));
							}
						}
						
						FPlatformApplicationMisc::ClipboardCopy(*Clipboard);

						FNotificationInfo Info(LOCTEXT("StallLogInfoCopied", "Copied to clipboard"));
						Info.ExpireDuration = 2.0f;
						FSlateNotificationManager::Get().AddNotification(Info);
						
						return FReply::Handled();
					})
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush("GenericCommands.Copy"))
						.DesiredSizeOverride(FVector2D(16.0f, 16.f))
					]
				]
				+SHorizontalBox::Slot()
				[
					SNew(SSpacer)
				];
			}

			return SNullWidget::NullWidget;
		}

	private:
		FStallLogItemPtr StallLogItem;
	};

	/**
	 * A widget to display the table of stalls
	 */
	class SStallLog
		: public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS( SStallLog ) {}
			SLATE_EVENT(FStallLogClearLog, OnClearLog)
			SLATE_ARGUMENT (const TArray<TSharedPtr<FStallLogItem>>*, StallLogItems)
		SLATE_END_ARGS()

		virtual ~SStallLog();

		void Construct(const FArguments& InArgs);

	private:
		
		TSharedPtr<SListView<FStallLogItemPtr>> ListViewPtr;
		FStallLogClearLog ClearLogDelegate;
	};

	SStallLog::~SStallLog()	= default;
	
	void SStallLog::Construct(const FArguments& InArgs)
	{
		auto StallLogGenerateRow = [](FStallLogItemPtr StallLogItem, const TSharedRef<STableViewBase>& OwnerTable) -> TSharedRef<ITableRow>
		{
			return SNew(SStallLogItemRow, OwnerTable, StallLogItem);
		};

		ClearLogDelegate = InArgs._OnClearLog;

		ChildSlot
		.Padding(3)
		[
			SNew(SVerticalBox)

			// Table
			+SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(EVerticalAlignment::VAlign_Top)
			.Padding(FMargin(0.0f, 4.0f, 0.0f, 4.0f))
			[
				SNew(SSplitter)
				.PhysicalSplitterHandleSize(2.0f)
				.ResizeMode(ESplitterResizeMode::FixedSize)
				+ SSplitter::Slot()
				.Value(0.15f)
				[
					SNew(SBox)
					.Padding(FMargin(4.f))
					[
						SNew(SBorder)
						.Padding(FMargin(0))
						.BorderImage(FAppStyle::GetBrush("Brushes.Recessed"))
						[
							SAssignNew(ListViewPtr, SListView<TSharedPtr<FStallLogItem>>)
								.ListItemsSource(InArgs._StallLogItems)
								.OnGenerateRow_Lambda(StallLogGenerateRow)
								.SelectionMode(ESelectionMode::None)
								.HeaderRow
								(
									SNew(SHeaderRow)

									+ SHeaderRow::Column(ColumnName_Location)
										.DefaultLabel(LOCTEXT("StallLogColumnHeader_StallDetectorName", "Stall Detector Name"))
										.FillWidth(0.3f)

									+ SHeaderRow::Column(ColumnName_Thread)
										  .DefaultLabel(LOCTEXT("StallLogColumnHeader_ThreadName", "Thread Name"))
										  .FillWidth(0.3f)

									+ SHeaderRow::Column(ColumnName_Duration)
										.DefaultLabel(LOCTEXT("StallLogColumnHeader_Duration", "Duration"))
										.FillWidth(0.3f)

									+ SHeaderRow::Column(ColumnName_Time)
										.DefaultLabel(LOCTEXT("StallLogColumnHeader_Time", "Time Of Stall"))
										.FillWidth(0.3f)

									+ SHeaderRow::Column(ColumnName_Copy)
										.DefaultLabel(LOCTEXT("StallLogColumnHeader_CopyButton", "Copy Stall Info"))
										.FillWidth(0.1f)
								)
								
						]
					]
				]
			]

			+SVerticalBox::Slot()
			[
				SNew(SSpacer)
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(EVerticalAlignment::VAlign_Bottom)
			[
				SNew(SButton)
				.OnClicked_Lambda([this]()
				{
					const bool Executed = ClearLogDelegate.ExecuteIfBound();
					return Executed ? FReply::Handled() : FReply::Unhandled();
				})
				[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("StallLog_Clear", "Clear Stall Log"))
					.Justification(ETextJustify::Center)
				]
				]
			]
		];

	}
}

void FStallLogHistory::OnStallDetected(
	uint64 UniqueID,
	FDateTime InDetectTime,
	TConstArrayView<uint64> Backtrace)
{
	checkf(IsInGameThread(), TEXT("Can only be run on GameThread"));

	FStallLogItem Existing;
	if (InFlightStalls.RemoveAndCopyValue(UniqueID, Existing))
	{
		// Already handled completed event, just add back trace & detect time
		AddStallToLog(MakeShared<FStallLogItem>(MoveTemp(Existing), InDetectTime, Backtrace));
	}
	else
	{
		// Wait for completed event to finish with full duration
		InFlightStalls.Add(UniqueID, FStallLogItem(InDetectTime, Backtrace));
	}
}

void FStallLogHistory::OnStallCompleted(uint32 ThreadID, FStringView StatName, uint64 UniqueID, FDateTime InCompletedTime, float InDurationSeconds, bool bWasDetectedWithBacktrace)
{
	checkf(IsInGameThread(), TEXT("Can only be run on GameThread"));

	FString ThreadName = FThreadManager::GetThreadName(ThreadID);
	if (bWasDetectedWithBacktrace)
	{
		FStallLogItem Existing;
		if (InFlightStalls.RemoveAndCopyValue(UniqueID, Existing))
		{
			// Fill in final duration and add to list
			// Existing->DurationSeconds = InDurationSeconds;
			AddStallToLog(MakeShared<FStallLogItem>(MoveTemp(Existing), StatName, MoveTemp(ThreadName), InDurationSeconds));
		}
		else
		{
			// Still waiting for callstack, add full duration
			InFlightStalls.Add(UniqueID,
				FStallLogItem(StatName, FThreadManager::GetThreadName(ThreadID), InDurationSeconds));
		}
	}
	else
	{
		// No backtrace for this event
		AddStallToLog(MakeShared<FStallLogItem>(
			StatName,
			MoveTemp(ThreadName),
			InDurationSeconds,
			InCompletedTime));
	}
}

void FStallLogHistory::AddStallToLog(FStallLogItemPtr LogItemToAdd)
{
	StallLogs.Emplace(LogItemToAdd);
	SetHasViewedStalls(false);
}

DECLARE_CYCLE_STAT(
	TEXT("StallLoggerSubsystem"),
	STAT_FDelegateGraphTask_StallLogger,
	STATGROUP_TaskGraphTasks);

namespace
{	
	void RegisterStallsLogListing()
	{
		FMessageLogInitializationOptions InitOptions;
		InitOptions.bShowFilters = true;
		InitOptions.bAllowClear = true;

		FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
		MessageLogModule.RegisterLogListing("StallLog", LOCTEXT("StallLog", "Editor Stall Logger"), InitOptions);
	}

	void UnregisterStallsLogListing()
	{
		FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
		MessageLogModule.UnregisterLogListing("StallLog");
	}
}

bool UStallLogSubsystem::ShouldCreateSubsystem(UObject* Outer) const
{
#if !WITH_EDITOR
	return false;
#endif
	
	if (!FSlateApplication::IsInitialized())
	{
		return false;
	}
	
	return Super::ShouldCreateSubsystem(Outer);
}

void UStallLogSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	UEditorSubsystem::Initialize(Collection);

	StallLogHistory = MakeShared<FStallLogHistory>();

	RegisterStallsLogListing();
	RegisterStallDetectedDelegates();
}

void UStallLogSubsystem::Deinitialize()
{
	UnregisterStallDetectedDelegates();
	UnregisterStallsLogListing();
}
	
TSharedRef<SWidget> UStallLogSubsystem::CreateStallLogPanel()
{
	StallLogHistory->SetHasViewedStalls(true);

	TSharedRef<SWidget> StallLog = SNew(SStallLog)
	.StallLogItems(&StallLogHistory->GetStallLog())
	.OnClearLog_Lambda([StallLogHistoryWkPtr = this->StallLogHistory.ToWeakPtr()]
	{
		TSharedPtr<FStallLogHistory> StallLogHistoryPtr = StallLogHistoryWkPtr.Pin();
		if (StallLogHistoryPtr)
		{
			StallLogHistoryPtr->ClearStallLog();
		}
	});

	return StallLog;
}

void UStallLogSubsystem::RegisterStallDetectedDelegates()
{
	// Only compile this if StallDetector exists
	// Note: This is not preferred, however the implementation relies on StallDetector and definitions are stripped
	// on many configurations
#if STALL_DETECTOR
	
	OnStallDetectedDelegate = UE::FStallDetector::StallDetected.AddLambda(
			[StallLogHistory = this->StallLogHistory](const UE::FStallDetectedParams& Params)
			{
				if (!EnableStallLogSubsystem)
				{
					return;
				}

				FDateTime Now = FDateTime::Now();

				// Add a bookmark to Insights when a stall is detected
				// Helps understand the context of the stall on the thread timeline
				TRACE_BOOKMARK(TEXT("Stall [%s]"), *FString(Params.StatName));

				FFunctionGraphTask::CreateAndDispatchWhenReady(
					[StallLogHistory,
						UniqueID = Params.UniqueID,
						Backtrace = Params.Backtrace,
						StatName = FText::FromStringView(Params.StatName),
						Now,
						ThreadID = Params.ThreadId]() mutable {
						check(IsInGameThread())

							StallLogHistory->OnStallDetected(UniqueID, Now, Backtrace);
					},
					GET_STATID(STAT_FDelegateGraphTask_StallLogger),
					nullptr,
					ENamedThreads::GameThread);
			});
	
	OnStallCompletedDelegate = UE::FStallDetector::StallCompleted.AddLambda(
			[StallLogHistory = this->StallLogHistory](const UE::FStallCompletedParams& Params)
			{
				if (!EnableStallLogSubsystem)
				{
					return;
				}
				
				// Log the end event to the message log. Make sure we're doing that from the game thread.
				const FGraphEventArray* Prerequisites = nullptr;

				FGraphEventRef LogStallEndAsyncTask_GameThread(
					FFunctionGraphTask::CreateAndDispatchWhenReady(
						[Params,
							Now = FDateTime::Now(),
							StallLogHistory]() {
							check(IsInGameThread());

							FMessageLog MessageLog("StallLog");

							StallLogHistory->OnStallCompleted(
								Params.ThreadId,
								Params.StatName,
								Params.UniqueID,
								Now,
								float( Params.BudgetSeconds + Params.OverbudgetSeconds ),
								Params.bWasTriggered);
						},
						GET_STATID(STAT_FDelegateGraphTask_StallLogger),
						Prerequisites,
						ENamedThreads::GameThread));
			}
		);
#endif // STALL_DETECTOR
}

void UStallLogSubsystem::UnregisterStallDetectedDelegates()
{
#if STALL_DETECTOR
	UE::FStallDetector::StallDetected.Remove(OnStallCompletedDelegate);
	OnStallCompletedDelegate.Reset();
	
	UE::FStallDetector::StallDetected.Remove(OnStallCompletedDelegate);
	OnStallCompletedDelegate.Reset();
#endif // STALL_DETECTOR
}

namespace UE::Debug
{
	static void StallCommand(const TArray<FString>& Arguments)
	{
		float SecondsToStall = 2.0;
		if (Arguments.Num() >= 1)
		{
			LexFromString(SecondsToStall, *Arguments[0]);
		}
	
		FFunctionGraphTask::CreateAndDispatchWhenReady(
			[SecondsToStall]()
			{
				SCOPED_NAMED_EVENT_TEXT(TEXT("Fake Stall"), FColor::Red);
				SCOPE_STALL_COUNTER(FakeStall, 1.0f);
			
				const float StartTime = (float)FPlatformTime::Seconds();
				FPlatformProcess::SleepNoStats(SecondsToStall);
			
				while (FPlatformTime::Seconds() - StartTime < SecondsToStall)
				{
					// Busy wait the rest if not slept long enough
				}
			}, TStatId(), nullptr, ENamedThreads::AnyThread);
	}

	static void StallAndReportCommand(const TArray<FString>& Arguments)
	{
		float SecondsToStall = 2.0;
		if (Arguments.Num() >= 1)
		{
			LexFromString(SecondsToStall, *Arguments[0]);
		}
	
		FFunctionGraphTask::CreateAndDispatchWhenReady(
			[SecondsToStall]()
			{
				SCOPED_NAMED_EVENT_TEXT(TEXT("Fake Stall"), FColor::Red);
				SCOPE_STALL_REPORTER_ALWAYS(FakeStall, 1.0f);
			
				const double StartTime = FPlatformTime::Seconds();
				FPlatformProcess::SleepNoStats(SecondsToStall);
			
				while (FPlatformTime::Seconds() - StartTime < SecondsToStall)
				{
					// Busy wait the rest if not slept long enough
				}
			}, TStatId(), nullptr, ENamedThreads::AnyThread);
	}

	static FAutoConsoleCommand CmdEditorStallLoggingStall(
		TEXT("Editor.Performance.Debug.Stall"),
		TEXT("Runs a busy loop on the calling thread. Can pass a number of seconds to stall for in parameter (defaults to 2 seconds)."),
		FConsoleCommandWithArgsDelegate::CreateStatic(&StallCommand)
	);

	static FAutoConsoleCommand CmdEditorStallLoggingStallAndReport(
		TEXT("Editor.Performance.Debug.StallAndReport"),
		TEXT("Runs a busy loop on the calling thread. Can pass a number of seconds to stall for in parameter (defaults to 2 seconds). Will report stall to CRC"),
		FConsoleCommandWithArgsDelegate::CreateStatic(&StallAndReportCommand)
	);
}

#undef LOCTEXT_NAMESPACE
