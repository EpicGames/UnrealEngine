// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorSubsystem.h"

#include "Containers/ContainersFwd.h"
#include "Delegates/IDelegateInstance.h"
#include "Templates/SharedPointer.h"

#include "StallLogSubsystem.generated.h"

#define UE_API STALLLOGSUBSYSTEM_API

class FDelegateHandle;
class FSpawnTabArgs;
class FStallLogHistory;
struct FSlateBrush;
class SDockTab;
class SWidget;

/**
 * Subsystem that provides feedback on stall detection
 */
UCLASS(MinimalAPI)
class UStallLogSubsystem  : public UEditorSubsystem
{
	GENERATED_BODY()

public:
	UE_API virtual bool ShouldCreateSubsystem(UObject* Outer) const override;
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual void Deinitialize() override;

	UE_API TSharedRef<SWidget> CreateStallLogPanel();

private:
	void RegisterStallDetectedDelegates();
	void UnregisterStallDetectedDelegates();

private:

	FDelegateHandle OnStallDetectedDelegate;
	FDelegateHandle OnStallCompletedDelegate;
	TSharedPtr<FStallLogHistory, ESPMode::ThreadSafe> StallLogHistory;
};

#undef UE_API
