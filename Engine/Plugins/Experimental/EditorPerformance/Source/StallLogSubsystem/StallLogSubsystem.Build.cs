// Copyright Epic Games, Inc. All Rights Reserved.

// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class StallLogSubsystem : ModuleRules
{
	public StallLogSubsystem(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"ApplicationCore",
				"CoreUObject",
				"Engine",
				"EditorSubsystem",
				"InputCore",
				"MessageLog",
				"SlateCore",
				"Slate",
				"ToolMenus",
				"WorkspaceMenuStructure"
			}
		);

		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
	}
}